{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "LibVpxCodecs",
	"Description": "Adds codecs from LibVpx to AVCodecs",
	"Category": "Codecs",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "LibVpxCodecs",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"TargetDenyList": [
				"Server"
			],
			"PlatformAllowList": [
				"Win64",
                "Linux",
                "Mac",
				"Android",
				"IOS"
			]
		}
	],
	"Plugins": [
		{
			"Name": "AVCodecsCore",
			"Enabled": true
		}
	],
	"SupportedTargetPlatforms": [
		"Win64",
        "Linux",
        "Mac",
		"Android",
		"IOS"
	]
}