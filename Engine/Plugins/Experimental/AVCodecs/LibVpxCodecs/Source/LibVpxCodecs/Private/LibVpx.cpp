// Copyright Epic Games, Inc. All Rights Reserved.

#include "LibVpx.h"

REGISTER_TYPEID(FLibVpx);

bool FLibVpx::IsValid() const
{
	return true;
}
