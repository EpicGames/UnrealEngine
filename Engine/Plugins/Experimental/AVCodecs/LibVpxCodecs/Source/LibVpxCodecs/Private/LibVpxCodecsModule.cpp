// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "Video/Decoders/Configs/VideoDecoderConfigVP8.h"
#include "Video/Decoders/Configs/VideoDecoderConfigVP9.h"
#include "Video/Encoders/Configs/VideoEncoderConfigVP8.h"
#include "Video/Encoders/Configs/VideoEncoderConfigVP9.h"

#include "Video/Resources/VideoResourceCPU.h"

#include "Video/Decoders/VideoDecoderLibVpxVP8.h"
#include "Video/Decoders/VideoDecoderLibVpxVP9.h"
#include "Video/Encoders/VideoEncoderLibVpxVP8.h"
#include "Video/Encoders/VideoEncoderLibVpxVP9.h"

class FLibVpxCodecModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		// clang-format off
		FVideoEncoder
            ::RegisterPermutationsOf<TVideoEncoderLibVpxVP8<FVideoResourceCPU>>
            ::With<FVideoResourceCPU>
            ::And<FVideoEncoderConfigLibVpx, FVideoEncoderConfigVP8>(
                [](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) 
                {
			        return FAPI::Get<FLibVpx>().IsValid();
		        });

			FVideoEncoder
            ::RegisterPermutationsOf<TVideoEncoderLibVpxVP9<FVideoResourceCPU>>
            ::With<FVideoResourceCPU>
            ::And<FVideoEncoderConfigLibVpx, FVideoEncoderConfigVP9>(
                [](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) 
                {
			        return FAPI::Get<FLibVpx>().IsValid();
		        });

		FVideoDecoder
            ::RegisterPermutationsOf<TVideoDecoderLibVpxVP8<FVideoResourceCPU>>
            ::With<FVideoResourceCPU>
            ::And<FVideoDecoderConfigLibVpx, FVideoDecoderConfigVP8>(
                [](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) 
                {
			        return FAPI::Get<FLibVpx>().IsValid();
		        });

		FVideoDecoder
            ::RegisterPermutationsOf<TVideoDecoderLibVpxVP9<FVideoResourceCPU>>
            ::With<FVideoResourceCPU>
            ::And<FVideoDecoderConfigLibVpx, FVideoDecoderConfigVP9>(
                [](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) 
                {
			        return FAPI::Get<FLibVpx>().IsValid();
		        });
		// clang-format on
	}
};

IMPLEMENT_MODULE(FLibVpxCodecModule, LibVpxCodecs);
