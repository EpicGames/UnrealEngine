// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Decoders/Configs/VideoDecoderConfigLibVpx.h"

#include "Video/Decoders/Configs/VideoDecoderConfigVP8.h"
#include "Video/Decoders/Configs/VideoDecoderConfigVP9.h"

REGISTER_TYPEID(FVideoDecoderConfigLibVpx);

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigLibVpx& OutConfig, FVideoDecoderConfig const& InConfig)
{
	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfig& OutConfig, FVideoDecoderConfigLibVpx const& InConfig)
{
	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigLibVpx& OutConfig, FVideoDecoderConfigVP8 const& InConfig)
{
	OutConfig.NumberOfCores = InConfig.NumberOfCores;

	return FAVExtension::TransformConfig<FVideoDecoderConfigLibVpx, FVideoDecoderConfig>(OutConfig, InConfig);
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigLibVpx& OutConfig, FVideoDecoderConfigVP9 const& InConfig)
{
	OutConfig.MaxOutputWidth = InConfig.MaxOutputWidth;
	OutConfig.MaxOutputHeight = InConfig.MaxOutputHeight;

	OutConfig.NumberOfCores = InConfig.NumberOfCores;

	return FAVExtension::TransformConfig<FVideoDecoderConfigLibVpx, FVideoDecoderConfig>(OutConfig, InConfig);
}
