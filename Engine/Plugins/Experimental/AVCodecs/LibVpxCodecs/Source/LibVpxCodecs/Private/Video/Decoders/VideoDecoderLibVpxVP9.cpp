// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Decoders/VideoDecoderLibVpxVP9.h"