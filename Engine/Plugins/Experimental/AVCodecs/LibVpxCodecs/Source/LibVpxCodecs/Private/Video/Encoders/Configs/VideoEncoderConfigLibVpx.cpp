// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/Configs/VideoEncoderConfigLibVpx.h"

#include "Video/Encoders/Configs/VideoEncoderConfigVP8.h"
#include "Video/Encoders/Configs/VideoEncoderConfigVP9.h"

REGISTER_TYPEID(FVideoEncoderConfigLibVpx);

static uint32	   DEFAULT_BITRATE_MIN = 100000;
static uint32	   DEFAULT_BITRATE_TARGET = 1000000;
static uint32	   DEFAULT_BITRATE_MAX = 10000000;
static const float QPRANGE = 63.f; // Range used by VP8 and VP9

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigLibVpx& OutConfig, FVideoEncoderConfig const& InConfig)
{
	OutConfig.Width = InConfig.Width;
	OutConfig.Height = InConfig.Height;
	OutConfig.Preset = InConfig.Preset;
	OutConfig.Framerate = InConfig.TargetFramerate;
	OutConfig.MinBitrate = InConfig.MinBitrate > -1 ? InConfig.MinBitrate : DEFAULT_BITRATE_MIN;
	OutConfig.TargetBitrate = InConfig.TargetBitrate > -1 ? InConfig.TargetBitrate : DEFAULT_BITRATE_TARGET;
	OutConfig.MaxBitrate = InConfig.MaxBitrate > -1 ? InConfig.MaxBitrate : DEFAULT_BITRATE_MAX;
	OutConfig.MaxQP = static_cast<int32>(QPRANGE * (1.0f - (InConfig.MinQuality / 100.0f)));
	OutConfig.MinQP = static_cast<int32>(QPRANGE * (1.0f - (InConfig.MaxQuality / 100.0f)));
	OutConfig.KeyframeInterval = InConfig.KeyframeInterval > 0 ? InConfig.KeyframeInterval : 0;
	OutConfig.ScalabilityMode = InConfig.ScalabilityMode;
	OutConfig.NumberOfSpatialLayers = InConfig.NumberOfSpatialLayers;
	OutConfig.NumberOfTemporalLayers = InConfig.NumberOfTemporalLayers;
	FMemory::Memcpy(OutConfig.SpatialLayers, InConfig.SpatialLayers, sizeof(FSpatialLayer) * Video::MaxSpatialLayers);
	OutConfig.NumberOfSimulcastStreams = InConfig.NumberOfSimulcastStreams;
	FMemory::Memcpy(OutConfig.SimulcastStreams, InConfig.SimulcastStreams, sizeof(FSpatialLayer) * Video::MaxSimulcastStreams);
	for (size_t si = 0; si < Video::MaxSpatialLayers; si++)
	{
		for (size_t ti = 0; ti < Video::MaxTemporalStreams; ti++)
		{
			if (InConfig.Bitrates[si][ti].IsSet())
			{
				OutConfig.Bitrates[si][ti] = InConfig.Bitrates[si][ti].GetValue();
			}
		}
	}

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfig& OutConfig, FVideoEncoderConfigLibVpx const& InConfig)
{
	OutConfig.Width = InConfig.Width;
	OutConfig.Height = InConfig.Height;
	OutConfig.Preset = InConfig.Preset;
	OutConfig.TargetFramerate = InConfig.Framerate;
	OutConfig.MinBitrate = InConfig.MinBitrate > -1 ? InConfig.MinBitrate : DEFAULT_BITRATE_MIN;
	OutConfig.TargetBitrate = InConfig.TargetBitrate > -1 ? InConfig.TargetBitrate : DEFAULT_BITRATE_TARGET;
	OutConfig.MaxBitrate = InConfig.MaxBitrate > -1 ? InConfig.MaxBitrate : DEFAULT_BITRATE_MAX;
	OutConfig.MinQuality = static_cast<int32>((1.0f - (InConfig.MaxQP / QPRANGE)) * 100.0f);
	OutConfig.MaxQuality = static_cast<int32>((1.0f - (InConfig.MinQP / QPRANGE)) * 100.0f);
	OutConfig.KeyframeInterval = InConfig.KeyframeInterval > 0 ? InConfig.KeyframeInterval : 0;
	OutConfig.ScalabilityMode = InConfig.ScalabilityMode;
	OutConfig.NumberOfSpatialLayers = InConfig.NumberOfSpatialLayers;
	OutConfig.NumberOfTemporalLayers = InConfig.NumberOfTemporalLayers;
	FMemory::Memcpy(OutConfig.SpatialLayers, InConfig.SpatialLayers, sizeof(FSpatialLayer) * Video::MaxSpatialLayers);
	OutConfig.NumberOfSimulcastStreams = InConfig.NumberOfSimulcastStreams;
	FMemory::Memcpy(OutConfig.SimulcastStreams, InConfig.SimulcastStreams, sizeof(FSpatialLayer) * Video::MaxSimulcastStreams);

	for (size_t si = 0; si < Video::MaxSpatialLayers; si++)
	{
		for (size_t ti = 0; ti < Video::MaxTemporalStreams; ti++)
		{
			if (InConfig.Bitrates[si][ti].IsSet())
			{
				OutConfig.Bitrates[si][ti] = InConfig.Bitrates[si][ti].GetValue();
			}
		}
	}

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigLibVpx& OutConfig, FVideoEncoderConfigVP8 const& InConfig)
{
	return FAVExtension::TransformConfig<FVideoEncoderConfigLibVpx, FVideoEncoderConfig>(OutConfig, InConfig);
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigLibVpx& OutConfig, FVideoEncoderConfigVP9 const& InConfig)
{
	OutConfig.NumberOfCores = InConfig.NumberOfCores;
	OutConfig.bDenoisingOn = InConfig.bDenoisingOn;
	OutConfig.bAdaptiveQpMode = InConfig.bAdaptiveQpMode;
	OutConfig.bAutomaticResizeOn = InConfig.bAutomaticResizeOn;
	OutConfig.bFlexibleMode = InConfig.bFlexibleMode;
	OutConfig.InterLayerPrediction = InConfig.InterLayerPrediction;

	return FAVExtension::TransformConfig<FVideoEncoderConfigLibVpx, FVideoEncoderConfig>(OutConfig, InConfig);
}
