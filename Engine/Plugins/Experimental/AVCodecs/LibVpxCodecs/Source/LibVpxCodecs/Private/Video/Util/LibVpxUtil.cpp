// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Util/LibVpxUtil.h"

void LibVpxUtil::CopyI420(const uint8_t* RESTRICT SrcY,
	int											  SrcStrideY,
	const uint8_t* RESTRICT						  SrcU,
	int											  SrcStrideU,
	const uint8_t* RESTRICT						  SrcV,
	int											  SrcStrideV,
	uint8_t* RESTRICT							  DstY,
	int											  DstStrideY,
	uint8_t* RESTRICT							  DstU,
	int											  DstStrideU,
	uint8_t* RESTRICT							  DstV,
	int											  DstStrideV,
	int											  Width,
	int											  Height)
{
	int HalfWidth = (Width + 1) >> 1;
	int HalfHeight = (Height + 1) >> 1;
	if ((!SrcY && DstY) || !SrcU || !SrcV || !DstU || !DstV || Width <= 0 || Height == 0)
	{
		return;
	}
	// Negative Height means invert the image.
	if (Height < 0)
	{
		Height = -Height;
		HalfHeight = (Height + 1) >> 1;
		SrcY = SrcY + (Height - 1) * SrcStrideY;
		SrcU = SrcU + (HalfHeight - 1) * SrcStrideU;
		SrcV = SrcV + (HalfHeight - 1) * SrcStrideV;
		SrcStrideY = -SrcStrideY;
		SrcStrideU = -SrcStrideU;
		SrcStrideV = -SrcStrideV;
	}

	if (DstY)
	{
		CopyPlane(SrcY, SrcStrideY, DstY, DstStrideY, Width, Height);
	}
	// Copy UV planes.
	CopyPlane(SrcU, SrcStrideU, DstU, DstStrideU, HalfWidth, HalfHeight);
	CopyPlane(SrcV, SrcStrideV, DstV, DstStrideV, HalfWidth, HalfHeight);
}

void LibVpxUtil::CopyPlane(const uint8_t* RESTRICT SrcY,
	int											   SrcStrideY,
	uint8_t* RESTRICT							   DstY,
	int											   DstStrideY,
	int											   Width,
	int											   Height)
{
	if (Width <= 0 || Height == 0)
	{
		return;
	}
	// Negative Height means invert the image.
	if (Height < 0)
	{
		Height = -Height;
		DstY = DstY + (Height - 1) * DstStrideY;
		DstStrideY = -DstStrideY;
	}
	// Coalesce rows.
	if (SrcStrideY == Width && DstStrideY == Width)
	{
		Width *= Height;
		Height = 1;
		SrcStrideY = DstStrideY = 0;
	}
	// Nothing to do.
	if (SrcY == DstY && SrcStrideY == DstStrideY)
	{
		return;
	}

	// Copy plane
	for (int y = 0; y < Height; ++y)
	{
		memcpy(DstY, SrcY, Width);
		SrcY += SrcStrideY;
		DstY += DstStrideY;
	}
}