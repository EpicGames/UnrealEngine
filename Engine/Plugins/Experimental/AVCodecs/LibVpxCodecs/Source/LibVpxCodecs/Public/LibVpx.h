// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVUtility.h"

#include "CoreMinimal.h"

THIRD_PARTY_INCLUDES_START
#include "vp8.h"
#include "vp8cx.h"
#include "vp8dx.h"
#include "vpx_codec.h"
#include "vpx_encoder.h"
#include "vpx_decoder.h"
#include "vpx_ext_ratectrl.h"
#include "vpx_frame_buffer.h"
#include "vpx_image.h"
#include "vpx_integer.h"

#define UE_API LIBVPXCODECS_API
THIRD_PARTY_INCLUDES_END

class FLibVpx : public FAPI
{
public:
	FLibVpx() = default;

	UE_API virtual bool IsValid() const override;
};

DECLARE_TYPEID(FLibVpx, LIBVPXCODECS_API);

#undef UE_API
