// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVExtension.h"
#include "Video/VideoDecoder.h"
#include "LibVpx.h"

struct FVideoDecoderConfigLibVpx : public FAVConfig
{
public:
	// Values are parsed from the bitstream during decoding
	uint32 MaxOutputWidth = 0;
	uint32 MaxOutputHeight = 0;

	int32 NumberOfCores = 0;

	FVideoDecoderConfigLibVpx()
		: FAVConfig()
	{
	}

	FVideoDecoderConfigLibVpx(FVideoDecoderConfigLibVpx const& Other)
	{
		*this = Other;
	}

	FVideoDecoderConfigLibVpx& operator=(FVideoDecoderConfigLibVpx const& Other)
	{
		FMemory::Memcpy(*this, Other);

		return *this;
	}

	bool operator==(FVideoDecoderConfigLibVpx const& Other) const
	{
		return this->MaxOutputWidth == Other.MaxOutputWidth
			&& this->MaxOutputHeight == Other.MaxOutputHeight
			&& this->NumberOfCores == Other.NumberOfCores;
	}

	bool operator!=(FVideoDecoderConfigLibVpx const& Other) const
	{
		return !(*this == Other);
	}
};

template <>
FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigLibVpx& OutConfig, struct FVideoDecoderConfig const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(struct FVideoDecoderConfig& OutConfig, FVideoDecoderConfigLibVpx const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigLibVpx& OutConfig, struct FVideoDecoderConfigVP8 const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigLibVpx& OutConfig, struct FVideoDecoderConfigVP9 const& InConfig);

DECLARE_TYPEID(FVideoDecoderConfigLibVpx, LIBVPXCODECS_API);
