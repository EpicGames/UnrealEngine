// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoDecoder.h"
#include "Video/Decoders/Configs/VideoDecoderConfigLibVpx.h"
#include "Video/Resources/VideoResourceCPU.h"

#include "Containers/Queue.h"
#include "HAL/Platform.h"
#include "LibVpx.h"

template <typename TResource>
class TVideoDecoderLibVpxVP8 : public TVideoDecoder<TResource, FVideoDecoderConfigLibVpx>
{
private:
	uint8			 bInitialized : 1;
	uint64			 FrameCount = 0;
	bool			 bIsOpen = false;
	vpx_codec_ctx_t* Decoder = nullptr;
	int				 PropagationCount = -1;
	bool			 bKeyFrameRequired = true;

	struct FFrame
	{
		TArray<uint8> RawData;
		int			  StrideY = 0;
		int			  StrideUV = 0;
		int			  Width = 0;
		int			  Height = 0;
		uint64		  FrameNumber = 0;
	};

	TQueue<FFrame> Frames;

public:
	TVideoDecoderLibVpxVP8() = default;
	virtual ~TVideoDecoderLibVpxVP8() override;

	virtual bool	  IsOpen() const override;
	virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) override;
	virtual void	  Close() override;

	bool IsInitialized() const;

	virtual FAVResult ApplyConfig() override;

	virtual FAVResult ReceiveFrame(TResolvableVideoResource<TResource>& InOutResource) override;

	virtual FAVResult SendPacket(FVideoPacket const& Packet) override;

private:
	FAVResult Destroy();
};

#include "VideoDecoderLibVpxVP8.hpp"
