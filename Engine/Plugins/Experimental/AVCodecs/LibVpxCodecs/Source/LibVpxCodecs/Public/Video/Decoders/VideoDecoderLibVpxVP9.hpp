// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/Resources/VideoResourceCPU.h"
#include "Video/Util/LibVpxUtil.h"

template <typename TResource>
TVideoDecoderLibVpxVP9<TResource>::~TVideoDecoderLibVpxVP9()
{
	Close();
}

template <typename TResource>
bool TVideoDecoderLibVpxVP9<TResource>::IsOpen() const
{
	return bIsOpen;
}

template <typename TResource>
FAVResult TVideoDecoderLibVpxVP9<TResource>::Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
{
	Close();

	TVideoDecoder<TResource, FVideoDecoderConfigLibVpx>::Open(NewDevice, NewInstance);

	FrameCount = 0;

	bIsOpen = true;

	return EAVResult::Success;
}

template <typename TResource>
void TVideoDecoderLibVpxVP9<TResource>::Close()
{
	bIsOpen = false;
}

template <typename TResource>
bool TVideoDecoderLibVpxVP9<TResource>::IsInitialized() const
{
	return Decoder != nullptr;
}

template <typename TResource>
FAVResult TVideoDecoderLibVpxVP9<TResource>::ApplyConfig()
{
	if (IsOpen())
	{
		FVideoDecoderConfigLibVpx const& PendingConfig = this->GetPendingConfig();
		if (this->AppliedConfig != PendingConfig)
		{
			if (IsInitialized())
			{
				if (PendingConfig.MaxOutputWidth > 0 && PendingConfig.MaxOutputHeight > 0)
				{
					FAVResult Result = Destroy();
					if (Result != EAVResult::Success)
					{
						return Result;
					}
				}
			}

			if (!IsInitialized())
			{
				if (Decoder == nullptr)
				{
					Decoder = new vpx_codec_ctx_t;
					memset(Decoder, 0, sizeof(*Decoder));
				}

				vpx_codec_dec_cfg_t VpxConfig;

				if (!(PendingConfig.MaxOutputWidth > 0 && PendingConfig.MaxOutputHeight > 0))
				{
					// Postpone configuring number of threads until resolution is known.
					VpxConfig.threads = 1;
				}
				else
				{
					// We want to use multithreading when decoding high resolution videos. But
					// not too many in order to avoid overhead when many stream are decoded
					// concurrently.
					// Set 2 thread as target for 1280x720 pixel count, and then scale up
					// linearly from there - but cap at physical core count.
					// For common resolutions this results in:
					// 1 for 360p
					// 2 for 720p
					// 4 for 1080p
					// 8 for 1440p
					// 18 for 4K
					int NumThreads = FMath::Max<uint32>(1, 2 * PendingConfig.MaxOutputWidth * PendingConfig.MaxOutputHeight / (1280 * 720));
					VpxConfig.threads = FMath::Min(PendingConfig.NumberOfCores, NumThreads);
				}

				vpx_codec_flags_t Flags = 0;
				vpx_codec_err_t	  VpxResult = VPX_CODEC_OK;

				VpxResult = ::vpx_codec_dec_init(Decoder, vpx_codec_vp9_dx(), &VpxConfig, Flags);
				if (VpxResult != VPX_CODEC_OK)
				{
					delete Decoder;
					Decoder = nullptr;

					FString ErrorString(::vpx_codec_error_detail(Decoder));
					return FAVResult(EAVResult::Error, FString::Printf(TEXT("Error executing ::vpx_codec_dec_init. Error: %d, Details: %s"), VpxResult, *ErrorString), TEXT("LibVpx"));
				}

				bInitialized = true;

				// Always start with a complete key frame.
				bKeyFrameRequired = true;
			}
		}

		return TVideoDecoder<TResource, FVideoDecoderConfigLibVpx>::ApplyConfig();
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("LibVpx"));
}

template <typename TResource>
FAVResult TVideoDecoderLibVpxVP9<TResource>::SendPacket(FVideoPacket const& Packet)
{
	if (IsOpen())
	{
		FVideoDecoderConfigLibVpx& PendingConfig = this->EditPendingConfig();
		if (Packet.bIsKeyframe)
		{
			UE::AVCodecCore::VP9::Header_t Header;

			FVideoDecoderConfigVP9 VP9Config;
			if (VP9Config.Parse(Packet, Header).IsSuccess())
			{
				PendingConfig.MaxOutputWidth = Header.FrameWidth;
				PendingConfig.MaxOutputHeight = Header.FrameHeight;
			}
			else
			{
				return FAVResult(EAVResult::Error, TEXT("Failed to parse VP9 header"), TEXT("LibVpx"));
			}
		}

		FAVResult AVResult = ApplyConfig();
		if (AVResult.IsNotSuccess())
		{
			return AVResult;
		}

		// Always start with a complete key frame.
		if (bKeyFrameRequired)
		{
			if (!Packet.bIsKeyframe)
			{
				return FAVResult(EAVResult::Error, TEXT("KeyFrame required"), TEXT("LibVpx"));
			}
			bKeyFrameRequired = false;
		}

		vpx_codec_iter_t Iter = NULL;
		vpx_image_t*	 Img;

		const uint8_t* Buffer = Packet.DataPtr.Get();
		if (Packet.DataSize == 0)
		{
			Buffer = nullptr; // Triggers full frame concealment.
		}

		vpx_codec_err_t VpxResult = ::vpx_codec_decode(Decoder, Buffer, Packet.DataSize, 0, VPX_DL_REALTIME);
		if (VpxResult != VPX_CODEC_OK)
		{
			FString ErrorString(::vpx_codec_error_detail(Decoder));
			return FAVResult(EAVResult::Error, FString::Printf(TEXT("Error executing ::vpx_codec_decode. Error: %d, Details: %s"), VpxResult, *ErrorString), TEXT("LibVpx"));
		}

		Img = vpx_codec_get_frame(Decoder, &Iter);
		int QP;
		VpxResult = ::vpx_codec_control(Decoder, VPXD_GET_LAST_QUANTIZER, &QP);
		if (VpxResult != VPX_CODEC_OK)
		{
			FString ErrorString(::vpx_codec_error_detail(Decoder));
			FAVResult::Log(EAVResult::Error, FString::Printf(TEXT("Error executing ::vpx_codec_control. Setting VPXD_GET_LAST_QUANTIZER, Error: %d, Details: %s"), VpxResult, *ErrorString), TEXT("LibVpx"));
		}

		if (Img == nullptr)
		{
			return EAVResult::PendingOutput;
		}

		FFrame Frame = {};
		Frame.Width = Img->d_w;
		Frame.Height = Img->d_h;
		Frame.RawData.SetNum(Frame.Width * Frame.Height + (2 * ((Frame.Width + 1) / 2) * ((Frame.Height + 1) / 2)));
		LibVpxUtil::CopyI420(
			Img->planes[VPX_PLANE_Y], Img->stride[VPX_PLANE_Y],
			Img->planes[VPX_PLANE_U], Img->stride[VPX_PLANE_U],
			Img->planes[VPX_PLANE_V], Img->stride[VPX_PLANE_V],
			Frame.RawData.GetData(), Frame.Width,
			Frame.RawData.GetData() + Frame.Width * Frame.Height, (Frame.Width + 1) / 2,
			Frame.RawData.GetData() + Frame.Width * Frame.Height + (((Frame.Width + 1) / 2) * ((Frame.Height + 1) / 2)), (Frame.Width + 1) / 2,
			Img->d_w, Img->d_h);
		Frame.FrameNumber = FrameCount++;

		Frames.Enqueue(Frame);

		return EAVResult::Success;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("LibVpx"));
}

template <typename TResource>
FAVResult TVideoDecoderLibVpxVP9<TResource>::ReceiveFrame(TResolvableVideoResource<TResource>& InOutResource)
{
	if (IsOpen())
	{
		FFrame Frame;
		if (Frames.Peek(Frame))
		{
			if (!InOutResource.Resolve(this->GetDevice(), FVideoDescriptor(EVideoFormat::YUV420, Frame.Width, Frame.Height)))
			{
				return FAVResult(EAVResult::ErrorResolving, TEXT("Failed to resolve frame resource"), TEXT("LibVpx"));
			}

			TSharedPtr<uint8> const CopiedData = MakeShareable(new uint8[Frame.RawData.Num()]);
			FMemory::BigBlockMemcpy(CopiedData.Get(), Frame.RawData.GetData(), Frame.RawData.Num());

			InOutResource->SetRaw(CopiedData);

			Frames.Pop();

			return EAVResult::Success;
		}

		return EAVResult::PendingInput;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("LibVpx"));
}

template <typename TResource>
FAVResult TVideoDecoderLibVpxVP9<TResource>::Destroy()
{
	EAVResult Result = EAVResult::Success;

	if (Decoder != nullptr)
	{
		if (bInitialized)
		{
			if (::vpx_codec_destroy(Decoder))
			{
				Result = EAVResult::Error;
			}
		}
		delete Decoder;
		Decoder = nullptr;
	}

	bInitialized = false;
	return Result;
}