// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVConstants.h"
#include "AVExtension.h"
#include "Video/VideoEncoder.h"
#include "Video/CodecUtils/CodecUtilsVP9.h"
#include "LibVpx.h"

using namespace UE::AVCodecCore::VP9;

struct FVideoEncoderConfigLibVpx : public FAVConfig
{
public:
	uint32			 Width = 1920;
	uint32			 Height = 1080;
	uint32			 Framerate = 60;
	int32			 MaxBitrate = 20000000;
	int32			 TargetBitrate = 10000000;
	int32			 MinBitrate = 5000000;
	TOptional<int32> Bitrates[Video::MaxSpatialLayers][Video::MaxTemporalStreams];
	uint32			 KeyframeInterval = 0;
	EVideoFormat	 PixelFormat;
	uint32			 MinQP;
	uint32			 MaxQP;

	int32				  NumberOfCores = 0;
	bool				  bDenoisingOn = false;
	bool				  bAdaptiveQpMode = false;
	bool				  bAutomaticResizeOn = false;
	bool				  bFlexibleMode = false;
	EInterLayerPrediction InterLayerPrediction = EInterLayerPrediction::Off;

	uint8		  NumberOfSpatialLayers = 1;
	uint8		  NumberOfTemporalLayers = 1;
	FSpatialLayer SpatialLayers[Video::MaxSpatialLayers];

	uint8		  NumberOfSimulcastStreams;
	FSpatialLayer SimulcastStreams[Video::MaxSimulcastStreams];

	EScalabilityMode ScalabilityMode = EScalabilityMode::None;

	FVideoEncoderConfigLibVpx()
		: FAVConfig()
	{
	}

	FVideoEncoderConfigLibVpx(FVideoEncoderConfigLibVpx const& Other)
	{
		*this = Other;
	}

	FVideoEncoderConfigLibVpx& operator=(FVideoEncoderConfigLibVpx const& Other)
	{
		FMemory::Memcpy(*this, Other);

		return *this;
	}

	bool operator==(FVideoEncoderConfigLibVpx const& Other) const
	{
		return this->Width == Other.Width
			&& this->Height == Other.Height
			&& this->Framerate == Other.Framerate
			&& this->MaxBitrate == Other.MaxBitrate
			&& this->TargetBitrate == Other.TargetBitrate
			&& this->MinBitrate == Other.MinBitrate
			&& this->KeyframeInterval == Other.KeyframeInterval
			&& this->PixelFormat == Other.PixelFormat
			&& this->MinQP == Other.MinQP
			&& this->MaxQP == Other.MaxQP
			&& this->NumberOfCores == Other.NumberOfCores
			&& this->bDenoisingOn == Other.bDenoisingOn
			&& this->bAdaptiveQpMode == Other.bAdaptiveQpMode
			&& this->bAutomaticResizeOn == Other.bAutomaticResizeOn
			&& this->bFlexibleMode == Other.bFlexibleMode
			&& this->InterLayerPrediction == Other.InterLayerPrediction
			&& this->NumberOfSpatialLayers == Other.NumberOfSpatialLayers
			&& this->NumberOfTemporalLayers == Other.NumberOfTemporalLayers
			&& this->NumberOfSimulcastStreams == Other.NumberOfSimulcastStreams
			&& this->ScalabilityMode == Other.ScalabilityMode
			&& this->SameBitrates(Other.Bitrates)
			&& this->SameSpatialLayers(Other.SpatialLayers)
			&& this->SameSimulcastStreams(Other.SimulcastStreams);
	}

	bool operator!=(FVideoEncoderConfigLibVpx const& Other) const
	{
		return !(*this == Other);
	}

	bool SameBitrates(const TOptional<int32> OtherBitrates[Video::MaxSpatialLayers][Video::MaxTemporalStreams]) const
	{
		for (size_t si = 0; si < Video::MaxSpatialLayers; si++)
		{
			for (size_t ti = 0; ti < Video::MaxTemporalStreams; ti++)
			{
				if (Bitrates[si][ti] != OtherBitrates[si][ti])
				{
					return false;
				}
			}
		}

		return true;
	}

	bool SameSpatialLayers(const FSpatialLayer OtherSpatialLayers[Video::MaxSpatialLayers]) const
	{
		for (size_t si = 0; si < Video::MaxSpatialLayers; si++)
		{
			if (SpatialLayers[si] != OtherSpatialLayers[si])
			{
				return false;
			}
		}

		return true;
	}

	bool SameSimulcastStreams(const FSpatialLayer OtherSimulcastStreams[Video::MaxSimulcastStreams]) const
	{
		for (size_t si = 0; si < Video::MaxSimulcastStreams; si++)
		{
			if (SimulcastStreams[si] != OtherSimulcastStreams[si])
			{
				return false;
			}
		}

		return true;
	}
};

template <>
FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigLibVpx& OutConfig, struct FVideoEncoderConfig const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(struct FVideoEncoderConfig& OutConfig, FVideoEncoderConfigLibVpx const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigLibVpx& OutConfig, struct FVideoEncoderConfigVP8 const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigLibVpx& OutConfig, struct FVideoEncoderConfigVP9 const& InConfig);

DECLARE_TYPEID(FVideoEncoderConfigLibVpx, LIBVPXCODECS_API);
