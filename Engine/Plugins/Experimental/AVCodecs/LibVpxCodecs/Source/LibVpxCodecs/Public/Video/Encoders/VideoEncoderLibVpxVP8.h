// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/CodecUtils/CodecUtilsVP8.h"
#include "Video/VideoEncoder.h"
#include "Video/Encoders/Configs/VideoEncoderConfigLibVpx.h"
#include "Video/Resources/VideoResourceCPU.h"
#include "Video/Util/LibVpxUtil.h"

#include "Containers/Queue.h"
#include "HAL/Platform.h"
#include "LibVpx.h"

template <typename TResource>
class TVideoEncoderLibVpxVP8 : public TVideoEncoder<TResource, FVideoEncoderConfigLibVpx>
{
private:
	bool bIsOpen = false;

	int64_t	 Timestamp = 0;
	int		 QpMax = 56;
	int		 CpuSpeedDefault = -6;
	int		 NumCores = 0;
	uint32_t RCMaxIntraTarget = 0;
	int		 NumActiveStreams = 0;
	bool	 bKeyFrameRequest = false;
	bool	 bSendStream = false;
	int		 CpuSpeed = 0;
	uint64	 FrameCount = 0;

	TUniquePtr<vpx_image_t, LibVpxUtil::FImageDeleter> RawImage;
	TUniquePtr<vpx_codec_ctx_t, LibVpxUtil::FCodecContextDeleter> Encoder;
	TUniquePtr<vpx_codec_enc_cfg_t> VpxConfig;

	TQueue<FVideoPacket> Packets;

	TQueue<UE::AVCodecCore::VP8::FVP8FrameConfig> PendingFrames;

public:
	TVideoEncoderLibVpxVP8() = default;
	virtual ~TVideoEncoderLibVpxVP8() override;

	virtual bool	  IsOpen() const override;
	virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) override;
	virtual void	  Close() override;

	bool IsInitialized() const;

	virtual FAVResult ApplyConfig() override;

	virtual FAVResult SendFrame(TSharedPtr<FVideoResourceCPU> const& Resource, uint32 Timestamp, bool bForceKeyframe = false) override;

	virtual FAVResult ReceivePacket(FVideoPacket& OutPacket) override;

private:
	enum class EEncodeResult : uint8
	{
		Success,
		TargetBitrateOvershoot
	};

	FAVResult	  Destroy();
	uint32_t	  MaxIntraTarget(uint32_t OptimalBuffersize, uint32_t MaxFramerate);
	int			  GetCpuSpeed(uint32_t Width, uint32_t Height);
	int			  NumberOfThreads(uint32_t Width, uint32_t Height, int Cpus);
	FAVResult	  InitAndSetControlSettings(FVideoEncoderConfigLibVpx const& Config);
	void		  MaybeUpdatePixelFormat(vpx_img_fmt Format);
	EEncodeResult GetEncodedPartitions(uint32 Timestamp, bool bRetransmissionAllowed);
};

#include "VideoEncoderLibVpxVP8.hpp"
