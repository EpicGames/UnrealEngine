// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/Resources/VideoResourceCPU.h"
#include "Video/Util/LibVpxUtil.h"

using namespace UE::AVCodecCore::VP8;

#define SAFECONTROL(Encoder, Setting, Value)                                                                                                                                                         \
	{                                                                                                                                                                                                \
		vpx_codec_err_t Res = ::vpx_codec_control(Encoder, Setting, Value);                                                                                                                          \
		if (Res != VPX_CODEC_OK)                                                                                                                                                                     \
		{                                                                                                                                                                                            \
			FString ErrorString(::vpx_codec_error_detail(Encoder));                                                                                                                                  \
			FAVResult::Log(EAVResult::Success, FString::Printf(TEXT("Error executing ::vpx_codec_control. Setting %s, Error: %d, Details: %s"), TEXT(#Setting), Res, *ErrorString), TEXT("LibVpx")); \
		}                                                                                                                                                                                            \
	}

template <typename TResource>
TVideoEncoderLibVpxVP8<TResource>::~TVideoEncoderLibVpxVP8()
{
	Close();
}

template <typename TResource>
bool TVideoEncoderLibVpxVP8<TResource>::IsOpen() const
{
	return bIsOpen;
}

template <typename TResource>
FAVResult TVideoEncoderLibVpxVP8<TResource>::Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
{
	Close();

	TVideoEncoder<TResource, FVideoEncoderConfigLibVpx>::Open(NewDevice, NewInstance);

	FrameCount = 0;

	bIsOpen = true;

	return EAVResult::Success;
}

template <typename TResource>
void TVideoEncoderLibVpxVP8<TResource>::Close()
{
	Destroy();

	bIsOpen = false;
}

template <typename TResource>
bool TVideoEncoderLibVpxVP8<TResource>::IsInitialized() const
{
	return Encoder.IsValid();
}

template <typename TResource>
FAVResult TVideoEncoderLibVpxVP8<TResource>::ApplyConfig()
{
	if (IsOpen())
	{
		FVideoEncoderConfigLibVpx const& PendingConfig = this->GetPendingConfig();
		if (this->AppliedConfig != PendingConfig)
		{
			if (IsInitialized())
			{
				// Check if a setting has changed and destroy the encoder for re-creation.
				// If TargetBitrate, MinQuality, MaxQuality or Framerate has changed then only need to change the config.
				if (this->AppliedConfig.Width != PendingConfig.Width
					|| this->AppliedConfig.Height != PendingConfig.Height
					|| this->AppliedConfig.KeyframeInterval != PendingConfig.KeyframeInterval
					|| this->AppliedConfig.PixelFormat != PendingConfig.PixelFormat
					|| this->AppliedConfig.NumberOfCores != PendingConfig.NumberOfCores
					|| this->AppliedConfig.bDenoisingOn != PendingConfig.bDenoisingOn)
				{
					// Setting has changed. In that case we need to destroy and re-init
					Destroy();
					FAVResult::Log(EAVResult::Success, TEXT("Re-initializing encoding session"), TEXT("LibVpx"));
				}
				else if (
					this->AppliedConfig.TargetBitrate != PendingConfig.TargetBitrate
					|| this->AppliedConfig.MaxQP != PendingConfig.MaxQP
					|| this->AppliedConfig.MinQP != PendingConfig.MinQP)
				{
					bSendStream = PendingConfig.TargetBitrate > 0;
					VpxConfig->rc_target_bitrate = PendingConfig.TargetBitrate / 1000;

					VpxConfig->rc_min_quantizer = PendingConfig.MinQP;
					VpxConfig->rc_max_quantizer = PendingConfig.MaxQP;

					RCMaxIntraTarget = MaxIntraTarget(VpxConfig->rc_buf_optimal_sz, PendingConfig.Framerate);

					vpx_codec_err_t Result = ::vpx_codec_enc_config_set(Encoder.Get(), VpxConfig.Get());
					if (Result != VPX_CODEC_OK)
					{
						FString ErrorString(::vpx_codec_error_detail(Encoder.Get()));
						return FAVResult(EAVResult::Error, FString::Printf(TEXT("Error configuring codec, error code: %d, details: %s"), Result, *ErrorString), TEXT("LibVpx"));
					}
				}
				// Only change framerate if that is the only setting changed
				else if (this->AppliedConfig.Framerate != PendingConfig.Framerate)
				{
					RCMaxIntraTarget = MaxIntraTarget(VpxConfig->rc_buf_optimal_sz, PendingConfig.Framerate);
				}
			}

			if (!IsInitialized())
			{
				TOptional<vpx_img_fmt_t> PreviousImgFmt = RawImage ? RawImage->fmt : TOptional<vpx_img_fmt_t>();

				FAVResult Result = Destroy();
				if (Result != EAVResult::Success)
				{
					return Result;
				}

				if (!Encoder.IsValid())
				{
					Encoder = TUniquePtr<vpx_codec_ctx_t, LibVpxUtil::FCodecContextDeleter>(new vpx_codec_ctx_t, LibVpxUtil::FCodecContextDeleter());
					memset(Encoder.Get(), 0, sizeof(*(Encoder.Get())));
				}

				if (!VpxConfig.IsValid())
				{
					VpxConfig.Reset(new vpx_codec_enc_cfg_t);
					memset(VpxConfig.Get(), 0, sizeof(*(VpxConfig.Get())));
				}

				// TODO (william.belcher): Simulcast
				int NumStreams = 1;

				Timestamp = 0;
				bSendStream = true;
				bKeyFrameRequest = false;

				if (::vpx_codec_enc_config_default(vpx_codec_vp8_cx(), VpxConfig.Get(), 0))
				{
					return EAVResult::Error;
				}

				// TODO (william.belcher): Move all these magic values to config

				// setting the time base of the codec
				VpxConfig->g_timebase.num = 1;
				VpxConfig->g_timebase.den = 90000;
				VpxConfig->g_lag_in_frames = 0; // 0- no frame lagging

				VpxConfig->g_error_resilient = 0;

				VpxConfig->rc_dropframe_thresh = 30;
				VpxConfig->rc_end_usage = VPX_CBR;
				VpxConfig->g_pass = VPX_RC_ONE_PASS;
				// Handle resizing outside of libvpx.
				VpxConfig->rc_resize_allowed = 0;
				VpxConfig->rc_min_quantizer = PendingConfig.MinQP;
				VpxConfig->rc_max_quantizer = PendingConfig.MaxQP;
				VpxConfig->rc_undershoot_pct = 100;
				VpxConfig->rc_overshoot_pct = 15;
				VpxConfig->rc_buf_initial_sz = 500;
				VpxConfig->rc_buf_optimal_sz = 600;
				VpxConfig->rc_buf_sz = 1000;

				// Set the maximum target size of any key-frame.
				RCMaxIntraTarget = MaxIntraTarget(VpxConfig->rc_buf_optimal_sz, PendingConfig.Framerate);

				if (PendingConfig.KeyframeInterval > 0)
				{
					VpxConfig->kf_mode = VPX_KF_AUTO;
					VpxConfig->kf_max_dist = PendingConfig.KeyframeInterval;
				}
				else
				{
					VpxConfig->kf_mode = VPX_KF_DISABLED;
				}

				CpuSpeed = -6;
				CpuSpeedDefault = CpuSpeed;

				// Set encoding complexity (cpu_speed) based on resolution and/or platform.
				CpuSpeed = GetCpuSpeed(PendingConfig.Width, PendingConfig.Height);

				VpxConfig->g_w = PendingConfig.Width;
				VpxConfig->g_h = PendingConfig.Height;

				// Determine number of threads based on the image size and #cores.
				VpxConfig->g_threads = NumberOfThreads(VpxConfig->g_w, VpxConfig->g_h, PendingConfig.NumberOfCores);

				// Creating a wrapper to the image - setting image data to NULL.
				// Actual pointer will be set in encode. Setting align to 1, as it
				// is meaningless (no memory allocation is done here).
				RawImage = TUniquePtr<vpx_image_t, LibVpxUtil::FImageDeleter>(::vpx_img_wrap(nullptr, PreviousImgFmt.Get(VPX_IMG_FMT_I420), PendingConfig.Width, PendingConfig.Height, 1, NULL), LibVpxUtil::FImageDeleter());

				VpxConfig->rc_target_bitrate = PendingConfig.TargetBitrate / 1000;

				if (InitAndSetControlSettings(PendingConfig) != EAVResult::Success)
				{
					return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to create TVideoEncoderLibVpxVP8"), TEXT("LibVpx"));
				}
			}
		}

		return TVideoEncoder<TResource, FVideoEncoderConfigLibVpx>::ApplyConfig();
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("LibVpx"));
}

template <typename TResource>
FAVResult TVideoEncoderLibVpxVP8<TResource>::SendFrame(TSharedPtr<FVideoResourceCPU> const& Resource, uint32 InTimestamp, bool bForceKeyframe)
{
	if (!IsOpen())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("LibVpx"));
	}

	FAVResult AVResult = ApplyConfig();
	if (AVResult.IsNotSuccess())
	{
		return AVResult;
	}

	if (Resource.IsValid())
	{
		bool bKeyFrameRequested = bKeyFrameRequest && bSendStream;
		if (!bKeyFrameRequested)
		{
			bKeyFrameRequested = bForceKeyframe;
		}

		bool bSendKeyFrame = bKeyFrameRequested;
		bool bRetransmissionAllowed = true;

		PendingFrames.Enqueue(FVP8FrameConfig(EBufferFlags::ReferenceAndUpdate, EBufferFlags::None, EBufferFlags::None));

		vpx_enc_frame_flags_t Flags = bSendKeyFrame ? VPX_EFLAG_FORCE_KF : 0;

		MaybeUpdatePixelFormat(VPX_IMG_FMT_I420);

		int StrideY = Resource->GetWidth();
		int StrideUV = (Resource->GetWidth() + 1) / 2;

		int DataSizeY = StrideY * Resource->GetHeight();
		int DataSizeUV = StrideUV * ((Resource->GetHeight() + 1) / 2);

		RawImage->planes[VPX_PLANE_Y] = static_cast<uint8*>(Resource->GetRaw().Get());
		RawImage->planes[VPX_PLANE_U] = static_cast<uint8*>(Resource->GetRaw().Get()) + DataSizeY;
		RawImage->planes[VPX_PLANE_V] = static_cast<uint8*>(Resource->GetRaw().Get()) + DataSizeY + DataSizeUV;
		RawImage->stride[VPX_PLANE_Y] = StrideY;
		RawImage->stride[VPX_PLANE_U] = StrideUV;
		RawImage->stride[VPX_PLANE_V] = StrideUV;

		if (bSendKeyFrame)
		{
			bKeyFrameRequest = false;
		}

		SAFECONTROL(Encoder.Get(), VP8E_SET_FRAME_FLAGS, static_cast<int>(Flags));
		SAFECONTROL(Encoder.Get(), VP8E_SET_TEMPORAL_LAYER_ID, 0);

		uint32_t Duration = 90000 / this->AppliedConfig.Framerate;

		EEncodeResult EncodeResult = EEncodeResult::Success;
		int			  NumTries = 0;
		// If the first try returns TargetBitrateOvershoot
		// the frame must be reencoded with the same parameters again because
		// target bitrate is exceeded and encoder state has been reset.
		while (NumTries == 0 || (NumTries == 1 && EncodeResult == EEncodeResult::TargetBitrateOvershoot))
		{
			++NumTries;
			// Note we must pass 0 for `flags` field in encode call below since they are
			// set above in `libvpx_interface_->vpx_codec_control_` function for each
			// encoder/spatial layer.
			int Result = ::vpx_codec_encode(Encoder.Get(), RawImage.Get(), Timestamp, Duration, 0, VPX_DL_REALTIME);
			// Reset specific intra frame thresholds, following the key frame.
			if (bSendKeyFrame)
			{
				SAFECONTROL(Encoder.Get(), VP8E_SET_MAX_INTRA_BITRATE_PCT, RCMaxIntraTarget);
			}

			if (Result)
			{
				return EAVResult::Error;
			}

			// Examines frame timestamps only.
			EncodeResult = GetEncodedPartitions(InTimestamp, bRetransmissionAllowed);
		}

		Timestamp += Duration;

		return EAVResult::Success;
	}
	else
	{
		// Flush encoder
		return EAVResult::Success;
	}
}

template <typename TResource>
FAVResult TVideoEncoderLibVpxVP8<TResource>::ReceivePacket(FVideoPacket& OutPacket)
{
	if (IsOpen())
	{
		if (Packets.Dequeue(OutPacket))
		{
			return EAVResult::Success;
		}

		return EAVResult::PendingInput;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("LibVpx"));
}

template <typename TResource>
FAVResult TVideoEncoderLibVpxVP8<TResource>::Destroy()
{
	EAVResult Result = EAVResult::Success;

	if (Encoder.IsValid())
	{
		Encoder.Reset();
	}

	if (RawImage.IsValid())
	{
		RawImage.Reset();
	}

	if (VpxConfig.IsValid())
	{
		VpxConfig.Reset();
	}

	return Result;
}

template <typename TResource>
uint32_t TVideoEncoderLibVpxVP8<TResource>::MaxIntraTarget(uint32_t OptimalBuffersize, uint32_t MaxFramerate)
{
	// Set max to the optimal buffer level (normalized by target BR),
	// and scaled by a scale_par.
	// Max target size = scale_par * optimal_buffer_size * targetBR[Kbps].
	// This value is presented in percentage of perFrameBw:
	// perFrameBw = targetBR[Kbps] * 1000 / framerate.
	// The target in % is as follows:
	float	 ScalePar = 0.5;
	uint32_t TargetPct = OptimalBuffersize * ScalePar * MaxFramerate / 10;

	// Don't go below 3 times the per frame bandwidth.
	const uint32_t MinIntraTh = 300;
	return (TargetPct < MinIntraTh) ? MinIntraTh : TargetPct;
}

template <typename TResource>
int TVideoEncoderLibVpxVP8<TResource>::GetCpuSpeed(uint32_t Width, uint32_t Height)
{
	if (Width * Height < 352 * 288)
	{
		return (CpuSpeedDefault < -4) ? -4 : CpuSpeedDefault;
	}
	else
	{
		return CpuSpeedDefault;
	}
}

template <typename TResource>
int TVideoEncoderLibVpxVP8<TResource>::NumberOfThreads(uint32_t Width, uint32_t Height, int Cpus)
{
	if (Width * Height >= 1920 * 1080 && Cpus > 8)
	{
		return 8; // 8 threads for 1080p on high perf machines.
	}
	else if (Width * Height > 1280 * 960 && Cpus >= 6)
	{
		// 3 threads for 1080p.
		return 3;
	}
	else if (Width * Height > 640 * 480 && Cpus >= 3)
	{
		// Default 2 threads for qHD/HD, but allow 3 if core count is high enough,
		// as this will allow more margin for high-core/low clock machines or if
		// not built with highest optimization.
		if (Cpus >= 6)
		{
			return 3;
		}
		return 2;
	}
	else
	{
		// 1 thread for VGA or less.
		return 1;
	}
}

template <typename TResource>
FAVResult TVideoEncoderLibVpxVP8<TResource>::InitAndSetControlSettings(FVideoEncoderConfigLibVpx const& Config)
{
	vpx_codec_flags_t Flags = 0;
	Flags |= VPX_CODEC_USE_OUTPUT_PARTITION;

	if (::vpx_codec_enc_init(Encoder.Get(), vpx_codec_vp8_cx(), VpxConfig.Get(), Flags))
	{
		return EAVResult::ErrorCreating;
	}

	SAFECONTROL(Encoder.Get(), VP8E_SET_NOISE_SENSITIVITY, static_cast<unsigned int>(Config.bDenoisingOn ? EDenoiserState::DenoiserOnAdaptive : EDenoiserState::DenoiserOff));

	// Allow more screen content to be detected as static.
	SAFECONTROL(Encoder.Get(), VP8E_SET_STATIC_THRESHOLD, 1u);
	SAFECONTROL(Encoder.Get(), VP8E_SET_CPUUSED, CpuSpeed);
	SAFECONTROL(Encoder.Get(), VP8E_SET_TOKEN_PARTITIONS, static_cast<vp8e_token_partitions>(0));
	SAFECONTROL(Encoder.Get(), VP8E_SET_MAX_INTRA_BITRATE_PCT, RCMaxIntraTarget);
	SAFECONTROL(Encoder.Get(), VP8E_SET_SCREEN_CONTENT_MODE, 0u);

	return EAVResult::Success;
}

template <typename TResource>
void TVideoEncoderLibVpxVP8<TResource>::MaybeUpdatePixelFormat(vpx_img_fmt Format)
{
	if (RawImage->fmt == Format)
	{
		return;
	}
	auto Width = RawImage->d_w;
	auto Height = RawImage->d_h;
	::vpx_img_free(RawImage.Get());
	RawImage = TUniquePtr<vpx_image_t, LibVpxUtil::FImageDeleter>(::vpx_img_wrap(nullptr, Format, Width, Height, 1, NULL), LibVpxUtil::FImageDeleter());
}

template <typename TResource>
typename TVideoEncoderLibVpxVP8<TResource>::EEncodeResult TVideoEncoderLibVpxVP8<TResource>::GetEncodedPartitions(uint32 InTimestamp, bool bRetransmissionAllowed)
{
	vpx_codec_iter_t Iter = nullptr;

	const vpx_codec_cx_pkt_t* Packet = nullptr;

	size_t EncodedSize = 0;
	Packet = ::vpx_codec_get_cx_data(Encoder.Get(), &Iter);
	while (Packet != nullptr)
	{
		if (Packet->kind == VPX_CODEC_CX_FRAME_PKT)
		{
			EncodedSize += Packet->data.frame.sz;
		}

		Packet = ::vpx_codec_get_cx_data(Encoder.Get(), &Iter);
	}

	TArray<uint8_t> Buffer;
	Buffer.SetNum(EncodedSize);

	Iter = nullptr;
	size_t EncodedPos = 0;

	Packet = ::vpx_codec_get_cx_data(Encoder.Get(), &Iter);
	while (Packet != nullptr)
	{
		switch (Packet->kind)
		{
			case VPX_CODEC_CX_FRAME_PKT:
			{
				FMemory::Memcpy(&Buffer.GetData()[EncodedPos], Packet->data.frame.buf, Packet->data.frame.sz);
				EncodedPos += Packet->data.frame.sz;
				break;
			}
			default:
				break;
		}
		// End of frame

		if ((Packet->data.frame.flags & VPX_FRAME_IS_FRAGMENT) == 0)
		{
			break;
		}

		Packet = ::vpx_codec_get_cx_data(Encoder.Get(), &Iter);
	}

	if (bSendStream && Packet != nullptr)
	{
		if (Buffer.Num() > 0)
		{
			TSharedPtr<uint8> const CopiedData = MakeShareable(new uint8[Buffer.Num()]);
			FMemory::BigBlockMemcpy(CopiedData.Get(), Buffer.GetData(), Buffer.Num());

			int QP = 0;
			SAFECONTROL(Encoder.Get(), VP8E_GET_LAST_QUANTIZER_64, &QP);

			FCodecSpecificInfo CodecSpecificInfo = {};

			FCodecSpecificInfoVP8& Info = CodecSpecificInfo.CodecSpecific.VP8;
			Info.bNonReference = (Packet->data.frame.flags & VPX_FRAME_IS_DROPPABLE) != 0;
			Info.TemporalIdx = 0xff;
			Info.bLayerSync = false;
			Info.KeyIdx = -1;
			Info.bUseExplicitDependencies = true;
			Info.ReferencedBuffersCount = 0;
			Info.UpdatedBuffersCount = 0;

			using namespace UE::AVCodecCore::VP8;

			FVP8FrameConfig FrameConfig;
			PendingFrames.Dequeue(FrameConfig);

			bool bIsKeyframe = (Packet->data.frame.flags & VPX_FRAME_IS_KEY) != 0;
			for (int i = 0; i < static_cast<int>(EBufferType::Count); ++i)
			{
				if (!bIsKeyframe && FrameConfig.References(static_cast<EBufferType>(i)))
				{
					Info.ReferencedBuffers[Info.ReferencedBuffersCount++] = i;
				}

				if (bIsKeyframe || FrameConfig.Updates(static_cast<EBufferType>(i)))
				{
					Info.UpdatedBuffers[Info.UpdatedBuffersCount++] = i;
				}
			}

			FVideoPacket VideoPacket = FVideoPacket(CopiedData, Buffer.Num(), InTimestamp, ++FrameCount, (uint32_t)QP, bIsKeyframe);
			VideoPacket.CodecSpecificInfo = CodecSpecificInfo;
			Packets.Enqueue(VideoPacket);

			return EEncodeResult::Success;
		}
	}

	return EEncodeResult::Success;
}
