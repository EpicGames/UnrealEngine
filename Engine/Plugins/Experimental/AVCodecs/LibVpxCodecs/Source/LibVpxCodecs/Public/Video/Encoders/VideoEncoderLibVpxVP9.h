// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/CodecUtils/CodecUtilsVP9.h"
#include "Video/Encoders/Configs/VideoEncoderConfigLibVpx.h"
#include "Video/Encoders/SVC/ScalableVideoController.h"
#include "Video/Resources/VideoResourceCPU.h"
#include "Video/VideoEncoder.h"
#include "Video/Util/LibVpxUtil.h"

#include "Containers/Queue.h"
#include "HAL/Platform.h"
#include "LibVpx.h"

template <typename TResource>
class TVideoEncoderLibVpxVP9 : public TVideoEncoder<TResource, FVideoEncoderConfigLibVpx>
{
private:
	uint64 FrameCount = 0;

	TQueue<FVideoPacket> Packets;

	bool bIsOpen = false;

public:
	TVideoEncoderLibVpxVP9() = default;
	virtual ~TVideoEncoderLibVpxVP9() override;

	virtual bool	  IsOpen() const override;
	virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) override;
	virtual void	  Close() override;

	bool IsInitialized() const;

	virtual FAVResult ApplyConfig() override;

	virtual FAVResult SendFrame(TSharedPtr<FVideoResourceCPU> const& Resource, uint32 Timestamp, bool bTriggerKeyFrame = false) override;

	virtual FAVResult ReceivePacket(FVideoPacket& OutPacket) override;

private:
	FAVResult							 Destroy();
	uint32								 MaxIntraTarget(uint32 OptimalBuffersize, uint32 MaxFramerate);
	int									 NumberOfThreads(uint32 Width, uint32 Height, int Cpus);
	FAVResult							 InitAndSetControlSettings(FVideoEncoderConfigLibVpx const& Config);
	bool								 ExplicitlyConfiguredSpatialLayers(FVideoEncoderConfigLibVpx const& Config) const;
	void								 UpdatePerformanceFlags(FVideoEncoderConfigLibVpx const& Config);
	TUniquePtr<FScalableVideoController> CreateScalabilityStructureFromConfig(FVideoEncoderConfigLibVpx const& Config);
	bool								 SetSvcRates(FVideoEncoderConfigLibVpx const& Config, const FVideoBitrateAllocation& Allocation);
	void								 MaybeRewrapRawWithFormat(const vpx_img_fmt Format);
	TTuple<size_t, size_t>				 GetActiveLayers(const FVideoBitrateAllocation& Allocation);

	// Configures which spatial layers libvpx should encode according to
	// configuration provided by SvcController.
	void EnableSpatialLayer(int Sid);
	void DisableSpatialLayer(int Sid);
	void SetActiveSpatialLayers();

	void					   DeliverBufferedFrame(bool bEndOfPicture);
	bool					   PopulateCodecSpecific(FCodecSpecificInfo* CodecSpecificInfo, TOptional<int>* SpatialIdx, TOptional<int>* TemporalIdx, const vpx_codec_cx_pkt& Packet);
	void					   FillReferenceIndices(const vpx_codec_cx_pkt& Packet, const size_t PicNum, const bool bInterLayerPredicted, FCodecSpecificInfoVP9* Info);
	void					   UpdateReferenceBuffers(const vpx_codec_cx_pkt& Packet, size_t PicNum);
	vpx_svc_ref_frame_config_t SetReferences(TArray<FScalableVideoController::FLayerFrameConfig>& LayerFrames);
	vpx_svc_ref_frame_config_t SetReferences(bool bIsKeyPic, int FirstActiveSpatialLayerId);

private:
	struct FRefFrameBuffer
	{
		friend bool operator==(const FRefFrameBuffer& Lhs, const FRefFrameBuffer& Rhs)
		{
			return Lhs.PicNum == Rhs.PicNum && Lhs.SpatialLayerId == Rhs.SpatialLayerId && Lhs.TemporalLayerId == Rhs.TemporalLayerId;
		}

		size_t PicNum = 0;
		int	   SpatialLayerId = 0;
		int	   TemporalLayerId = 0;
	};

	struct FParameterSet
	{
		int BaseLayerSpeed = -1; // Speed setting for TL0.
		int HighLayerSpeed = -1; // Speed setting for TL1-TL3.
		//  0 = deblock all temporal layers (TL)
		//  1 = disable deblock for top-most TL
		//  2 = disable deblock for all TLs
		int	 DeblockMode = 0;
		bool bAllowDenoising = true;
	};

	struct FPerformanceFlags
	{
		// If false, a lookup will be made in `settings_by_resolution` base on the
		// highest currently active resolution, and the overall speed then set to
		// to the `base_layer_speed` matching that entry.
		// If true, each active resolution will have it's speed and deblock_mode set
		// based on it resolution, and the high layer speed configured for non
		// base temporal layer frames.
		bool bUsePerLayerSpeed = false;
		// Map from min pixel count to settings for that resolution and above.
		// E.g. if you want some settings A if below wvga (640x360) and some other
		// setting B at wvga and above, you'd use map {{0, A}, {230400, B}}.
		TSortedMap<int, FParameterSet> SettingsByResolution;
	};
	static FPerformanceFlags GetDefaultPerformanceFlags();

	enum class EFrameType : uint8
	{
		I,
		P
	};

	class FInputImage
	{
	public:
		FInputImage(uint32 Timestamp)
			: Timestamp(Timestamp)
		{
		}

		uint32 Timestamp;
	};

	class FEncodedImage
	{
	public:
		TArray<uint8> GetEncodedData()
		{
			return EncodedData;
		}

		void SetEncodedData(TArray<uint8> InEncodedData)
		{
			EncodedData = InEncodedData;
			Size = InEncodedData.Num();
		}

		size_t GetSize() const
		{
			return Size;
		}

		void SetSize(size_t NewSize)
		{
			// Allow set_size(0) even if we have no buffer.
			check(NewSize <= NewSize == 0 ? 0 : Capacity());
			Size = NewSize;
		}

	public:
		EFrameType	   FrameType = EFrameType::P;
		int32		   Width = 0;
		int32		   Height = 0;
		uint32		   Timestamp = 0;
		int32		   QP = -1;
		TOptional<int> SpatialIndex;
		TOptional<int> TemporalIndex;

	private:
		size_t Capacity() const
		{
			return EncodedData.Num();
		}

		TArray<uint8> EncodedData;
		size_t		  Size = 0; // Size of encoded frame data.
	};

private:
	EProfile				Profile = EProfile::Profile0;
	int64					Timestamp = 0;
	bool					bForceKeyFrame = true;
	size_t					PicsSinceKey = 0;
	uint8					NumTemporalLayers = 0;		// Number of configured TLs
	uint8					NumSpatialLayers = 0;		// Number of configured SLs
	uint8					NumActiveSpatialLayers = 0; // Number of actively encoded SLs
	uint8					FirstActiveLayer = 0;
	bool					bIsSvc = false;
	bool					bIsFlexibleMode = false;
	EInterLayerPrediction	InterLayerPrediction = EInterLayerPrediction::On;
	bool					bExternalRefControl = false;
	bool					bFullSuperframeDrop = true;
	bool					bLayerBuffering = false;
	bool					bFirstFrameInPicture = true;
	uint32					RCMaxIntraTarget = 0;
	FEncodedImage			EncodedImage;
	FGroupOfFramesInfo		Gof;
	FVideoBitrateAllocation CurrentBitrateAllocation;
	bool					bSsInfoNeeded = false;
	bool					bForceAllActiveLayers = false;
	bool					bVpxConfigChanged = true;
	FCodecSpecificInfo		CodecSpecific;

	// Performance flags, ordered by `min_pixel_count`.
	const FPerformanceFlags PerformanceFlags = GetDefaultPerformanceFlags();
	// Caching of of `speed_configs_`, where index i maps to the resolution as
	// specified in `codec_.spatialLayer[i]`.
	TArray<FParameterSet> PerformanceFlagsBySpatialIndex;

	TArray<FRefFrameBuffer>								RefBuf;
	TArray<FScalableVideoController::FLayerFrameConfig> LayerFrames;

	TUniquePtr<FScalableVideoController> SvcController = nullptr;
	TUniquePtr<FInputImage>				 InputImage = nullptr;

private:
	TUniquePtr<vpx_codec_ctx_t, LibVpxUtil::FCodecContextDeleter> Encoder = nullptr;
	TUniquePtr<vpx_codec_enc_cfg_t>								  VpxConfig = nullptr;
	TUniquePtr<vpx_image_t, LibVpxUtil::FImageDeleter>			  RawImage = nullptr;
	TUniquePtr<vpx_svc_extra_cfg_t>								  SvcParams = nullptr;
	TUniquePtr<vpx_svc_frame_drop_t>							  SvcDropFrame = nullptr;

public:
	void GetEncodedLayerFrame(const vpx_codec_cx_pkt* Packet);
};

namespace Internal
{
	template <typename TResource>
	inline void EncoderOutputCodedPacketCallback(vpx_codec_cx_pkt* Packet, void* UserData)
	{
		static_cast<TVideoEncoderLibVpxVP9<TResource>*>(UserData)->GetEncodedLayerFrame(Packet);
	}
} // namespace Internal

#include "VideoEncoderLibVpxVP9.hpp"
