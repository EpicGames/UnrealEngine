// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StaticBitArray.h"
#include "Video/Encoders/SVC/ScalabilityStructureFactory.h"
#include "Video/Encoders/SVC/VideoBitrateAllocatorSVC.h"
#include "Video/Encoders/SVC/ScalableVideoControllerNoLayering.h"
#include "Video/Resources/VideoResourceCPU.h"
#include "Video/Util/LibVpxUtil.h"
#include "Video/CodecUtils/CodecUtilsVP9.h"

using namespace UE::AVCodecCore::VP9;

#define SAFECONTROLVP9(Encoder, Setting, Value)                                                                                                                                                        \
	{                                                                                                                                                                                                  \
		vpx_codec_err_t Res = ::vpx_codec_control(Encoder, Setting, Value);                                                                                                                            \
		if (Res != VPX_CODEC_OK)                                                                                                                                                                       \
		{                                                                                                                                                                                              \
			FString ErrorString(::vpx_codec_error_detail(Encoder));                                                                                                                                    \
			FString SettingString(#Setting);                                                                                                                                                           \
			FAVResult::Log(EAVResult::Warning, FString::Printf(TEXT("Failed to execute ::vpx_codec_control. Setting %s, Error: %d, Details: %s"), *SettingString, Res, *ErrorString), TEXT("LibVpx")); \
		}                                                                                                                                                                                              \
	}

template <typename TResource>
typename TVideoEncoderLibVpxVP9<TResource>::FPerformanceFlags TVideoEncoderLibVpxVP9<TResource>::GetDefaultPerformanceFlags()
{
	FPerformanceFlags Flags;
	Flags.bUsePerLayerSpeed = true;
	// For smaller resolutions, use lower speed setting for the temporal base
	// layer (get some coding gain at the cost of increased encoding complexity).
	// Set encoder Speed 5 for TL0, encoder Speed 8 for upper temporal layers, and
	// disable deblocking for upper-most temporal layers.
	Flags.SettingsByResolution.Add(0, FParameterSet{ .BaseLayerSpeed = 5, .HighLayerSpeed = 8, .DeblockMode = 1, .bAllowDenoising = true });

	// Use speed 7 for QCIF and above.
	// Set encoder Speed 7 for TL0, encoder Speed 8 for upper temporal layers, and
	// enable deblocking for all temporal layers.
	Flags.SettingsByResolution.Add(352 * 288, FParameterSet{ .BaseLayerSpeed = 7, .HighLayerSpeed = 8, .DeblockMode = 0, .bAllowDenoising = true });

	// For very high resolution (1080p and up), turn the speed all the way up
	// since this is very CPU intensive. Also disable denoising to save CPU, at
	// these resolutions denoising appear less effective and hopefully you also
	// have a less noisy video source at this point.
	Flags.SettingsByResolution.Add(1920 * 1080, FParameterSet{ .BaseLayerSpeed = 9, .HighLayerSpeed = 9, .DeblockMode = 0, .bAllowDenoising = false });

	return Flags;
}

template <typename TResource>
TVideoEncoderLibVpxVP9<TResource>::~TVideoEncoderLibVpxVP9()
{
	Close();
}

template <typename TResource>
bool TVideoEncoderLibVpxVP9<TResource>::IsOpen() const
{
	return bIsOpen;
}

template <typename TResource>
FAVResult TVideoEncoderLibVpxVP9<TResource>::Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
{
	Close();

	TVideoEncoder<TResource, FVideoEncoderConfigLibVpx>::Open(NewDevice, NewInstance);

	FrameCount = 0;

	bIsOpen = true;

	return EAVResult::Success;
}

template <typename TResource>
void TVideoEncoderLibVpxVP9<TResource>::Close()
{
	Destroy();

	bIsOpen = false;
}

template <typename TResource>
bool TVideoEncoderLibVpxVP9<TResource>::IsInitialized() const
{
	return Encoder.IsValid();
}

template <typename TResource>
FAVResult TVideoEncoderLibVpxVP9<TResource>::ApplyConfig()
{
	if (IsOpen())
	{
		// TODO (Eden.Harris) RTCP-7994 Change this to const reference and GetPendingConfig when no longer changing the 
		// width/height. VP9 currently requires even width and height when rc_resize_allowed is enabled, otherwise it 
		// can crash when encoding P-Frames. 
		// libvpx vp9_setup_scale_factors_for_frame sets several function pointers that may crash when encoding and 
		// they are not set. vp9_setup_scale_factors_for_frame calls valid_ref_frame_size but will not set the function 
		// pointers if valid_ref_frame_size returns false. If a frame size has an odd width/height, for instance when 
		// rc_resize_allowed is enabled, the encoder may encode a frame at half the size. valid_ref_frame_size checks 
		// works by doubling the frame size and comparing it against the reference size. In the case where there is a 
		// odd frame that is 201px wide, half size frame is 100px, and the check doubles it to 200px which will be less 
		// than the reference frame of 201px. If the check succeeds in another iteration, it will set the function 
		// pointers for further frames and will not crash. 
		FVideoEncoderConfigLibVpx& PendingConfig = this->EditPendingConfig();
		if (PendingConfig.bAutomaticResizeOn)
		{
			// Round size down to even number. By rounding down, it saves copying the frame to a larger buffer.
			PendingConfig.Width = PendingConfig.Width & (~1);
			PendingConfig.Height = PendingConfig.Height & (~1);
		}
		if (this->AppliedConfig != PendingConfig)
		{
			if (IsInitialized())
			{
				if (this->AppliedConfig.Width == PendingConfig.Width
					&& this->AppliedConfig.Height == PendingConfig.Height
					&& this->AppliedConfig.KeyframeInterval == PendingConfig.KeyframeInterval
					&& this->AppliedConfig.PixelFormat == PendingConfig.PixelFormat
					&& this->AppliedConfig.MinQP == PendingConfig.MinQP
					&& this->AppliedConfig.MaxQP == PendingConfig.MaxQP
					&& this->AppliedConfig.NumberOfCores == PendingConfig.NumberOfCores
					&& this->AppliedConfig.bDenoisingOn == PendingConfig.bDenoisingOn
					&& this->AppliedConfig.bAdaptiveQpMode == PendingConfig.bAdaptiveQpMode
					&& this->AppliedConfig.bAutomaticResizeOn == PendingConfig.bAutomaticResizeOn
					&& this->AppliedConfig.bFlexibleMode == PendingConfig.bFlexibleMode
					&& this->AppliedConfig.InterLayerPrediction == PendingConfig.InterLayerPrediction
					&& this->AppliedConfig.NumberOfSpatialLayers == PendingConfig.NumberOfSpatialLayers
					&& this->AppliedConfig.NumberOfTemporalLayers == PendingConfig.NumberOfTemporalLayers
					&& this->AppliedConfig.NumberOfSimulcastStreams == PendingConfig.NumberOfSimulcastStreams
					&& this->AppliedConfig.ScalabilityMode == PendingConfig.ScalabilityMode)
				{
					FVideoBitrateAllocation NewAllocation;
					for (size_t si = 0; si < Video::MaxSpatialLayers; ++si)
					{
						for (size_t ti = 0; ti < Video::MaxTemporalStreams; ++ti)
						{
							NewAllocation.SetBitrate(si, ti, PendingConfig.Bitrates[si][ti].Get(0));
						}
					}

					if (!SetSvcRates(PendingConfig, NewAllocation))
					{
						FAVResult::Log(EAVResult::Warning, TEXT("Failed to set new bitrate allocation"), TEXT("LibVpx"));
					}
				}
				else
				{
					// Bitrate has stayed the same, but something else has changed. In that case we need to destroy and re-init
					Destroy();
					FAVResult::Log(EAVResult::Success, TEXT("Re-initializing encoding session"), TEXT("LibVpx"));
				}
			}

			if (!IsInitialized())
			{
				TOptional<vpx_img_fmt_t> PreviousImgFmt = RawImage ? RawImage->fmt : TOptional<vpx_img_fmt_t>();

				FAVResult Result = Destroy();
				if (Result != EAVResult::Success)
				{
					return Result;
				}

				if (!Encoder.IsValid())
				{
					Encoder = TUniquePtr<vpx_codec_ctx_t, LibVpxUtil::FCodecContextDeleter>(new vpx_codec_ctx_t, LibVpxUtil::FCodecContextDeleter());
					memset(Encoder.Get(), 0, sizeof(vpx_codec_ctx_t));
				}

				if (!VpxConfig.IsValid())
				{
					VpxConfig.Reset(new vpx_codec_enc_cfg_t);
					memset(VpxConfig.Get(), 0, sizeof(vpx_codec_enc_cfg_t));
				}

				if (!SvcParams.IsValid())
				{
					SvcParams.Reset(new vpx_svc_extra_cfg_t);
					memset(SvcParams.Get(), 0, sizeof(vpx_svc_extra_cfg_t));
				}

				Timestamp = 0;
				bForceKeyFrame = true;
				PicsSinceKey = 0;

				if (PendingConfig.ScalabilityMode != EScalabilityMode::None)
				{
					SvcController = CreateScalabilityStructure(PendingConfig.ScalabilityMode);
					if (!SvcController)
					{
						return FAVResult(EAVResult::Error, TEXT("Failed to create scalability structure"), TEXT("LibVpx"));
					}

					FScalableVideoController::FStreamLayersConfig Info = SvcController->StreamConfig();
					NumSpatialLayers = Info.NumSpatialLayers;
					NumTemporalLayers = Info.NumTemporalLayers;
					InterLayerPrediction = ScalabilityModeToInterLayerPredMode(PendingConfig.ScalabilityMode);
				}
				else
				{
					NumSpatialLayers = PendingConfig.NumberOfSpatialLayers;
					check(NumSpatialLayers > 0);
					NumTemporalLayers = PendingConfig.NumberOfTemporalLayers;
					if (NumTemporalLayers == 0)
					{
						NumTemporalLayers = 1;
					}
					InterLayerPrediction = PendingConfig.InterLayerPrediction;
					SvcController = CreateScalabilityStructureFromConfig(PendingConfig);
				}

				bIsSvc = (NumSpatialLayers > 1 || NumTemporalLayers > 1);

				vpx_codec_err_t Res = ::vpx_codec_enc_config_default(vpx_codec_vp9_cx(), VpxConfig.Get(), 0);
				if (Res != VPX_CODEC_OK)
				{
					return FAVResult(EAVResult::Error, FString::Printf(TEXT("Error executing ::vpx_codec_enc_config_default. Error: %d"), Res), TEXT("LibVpx"));
				}

				vpx_img_fmt_t PixelFormat = VPX_IMG_FMT_NONE;
				unsigned int  BitsForStorage = 8;
				switch (Profile)
				{
				case EProfile::Profile0:
					PixelFormat = PreviousImgFmt.Get(VPX_IMG_FMT_I420);
					BitsForStorage = 8;
					VpxConfig->g_bit_depth = VPX_BITS_8;
					VpxConfig->g_profile = 0;
					VpxConfig->g_input_bit_depth = 8;
					break;
				case EProfile::Profile1:
					// Encoding of profile 1 is not implemented. It would require extended
					// support for I444, I422, and I440 buffers.
					checkNoEntry();
					break;
				case EProfile::Profile2:
					PixelFormat = VPX_IMG_FMT_I42016;
					BitsForStorage = 16;
					VpxConfig->g_bit_depth = VPX_BITS_10;
					VpxConfig->g_profile = 2;
					VpxConfig->g_input_bit_depth = 10;
					break;
				case EProfile::Profile3:
					// Encoding of profile 3 is not implemented.
					checkNoEntry();
					break;
				}

				// Creating a wrapper to the image - setting image data to nullptr. Actual
				// pointer will be set in encode. Setting align to 1, as it is meaningless
				// (actual memory is not allocated).
				RawImage = TUniquePtr<vpx_image_t, LibVpxUtil::FImageDeleter>(::vpx_img_wrap(nullptr, PixelFormat, PendingConfig.Width, PendingConfig.Height, 1, NULL), LibVpxUtil::FImageDeleter());
				RawImage->bit_depth = BitsForStorage;

				// TODO (william.belcher): Move all these magic values to config

				VpxConfig->g_w = PendingConfig.Width;
				VpxConfig->g_h = PendingConfig.Height;
				VpxConfig->rc_target_bitrate = PendingConfig.MinBitrate;
				VpxConfig->g_error_resilient = bIsSvc ? VPX_ERROR_RESILIENT_DEFAULT : 0;
				// setting the time base of the codec
				VpxConfig->g_timebase.num = 1;
				VpxConfig->g_timebase.den = 90000;
				VpxConfig->g_lag_in_frames = 0; // 0- no frame lagging
				VpxConfig->g_threads = 1;
				VpxConfig->rc_dropframe_thresh = 0;
				VpxConfig->rc_end_usage = VPX_CBR;
				VpxConfig->g_pass = VPX_RC_ONE_PASS;
				VpxConfig->rc_min_quantizer = PendingConfig.MinQP;
				VpxConfig->rc_max_quantizer = PendingConfig.MaxQP;
				VpxConfig->rc_undershoot_pct = 50;
				VpxConfig->rc_overshoot_pct = 50;
				VpxConfig->rc_buf_initial_sz = 500;
				VpxConfig->rc_buf_optimal_sz = 600;
				VpxConfig->rc_buf_sz = 1000;
				// Set the maximum target size of any key-frame.
				RCMaxIntraTarget = MaxIntraTarget(VpxConfig->rc_buf_optimal_sz, PendingConfig.Framerate);
				// Key-frame interval is enforced manually by this wrapper.
				VpxConfig->kf_mode = VPX_KF_DISABLED;
				// TODO(webm:1592): work-around for libvpx issue, as it can still
				// put some key-frames at will even in VPX_KF_DISABLED kf_mode.
				VpxConfig->kf_max_dist = PendingConfig.KeyframeInterval;
				VpxConfig->kf_min_dist = VpxConfig->kf_max_dist;
				VpxConfig->rc_resize_allowed = PendingConfig.bAutomaticResizeOn ? 1 : 0;

				// Determine number of threads based on the image size and #cores.
				VpxConfig->g_threads = NumberOfThreads(VpxConfig->g_w, VpxConfig->g_h, PendingConfig.NumberOfCores);

				bIsFlexibleMode = PendingConfig.bFlexibleMode;

				bExternalRefControl = true;

				if (NumTemporalLayers == 1)
				{
					Gof.SetGofInfo(ETemporalStructureMode::TemporalStructureMode1);
					VpxConfig->temporal_layering_mode = VP9E_TEMPORAL_LAYERING_MODE_NOLAYERING;
					VpxConfig->ts_number_layers = 1;
					VpxConfig->ts_rate_decimator[0] = 1;
					VpxConfig->ts_periodicity = 1;
					VpxConfig->ts_layer_id[0] = 0;
				}
				else if (NumTemporalLayers == 2)
				{
					Gof.SetGofInfo(ETemporalStructureMode::TemporalStructureMode2);
					VpxConfig->temporal_layering_mode = VP9E_TEMPORAL_LAYERING_MODE_0101;
					VpxConfig->ts_number_layers = 2;
					VpxConfig->ts_rate_decimator[0] = 2;
					VpxConfig->ts_rate_decimator[1] = 1;
					VpxConfig->ts_periodicity = 2;
					VpxConfig->ts_layer_id[0] = 0;
					VpxConfig->ts_layer_id[1] = 1;
				}
				else if (NumTemporalLayers == 3)
				{
					Gof.SetGofInfo(ETemporalStructureMode::TemporalStructureMode3);
					VpxConfig->temporal_layering_mode = VP9E_TEMPORAL_LAYERING_MODE_0212;
					VpxConfig->ts_number_layers = 3;
					VpxConfig->ts_rate_decimator[0] = 4;
					VpxConfig->ts_rate_decimator[1] = 2;
					VpxConfig->ts_rate_decimator[2] = 1;
					VpxConfig->ts_periodicity = 4;
					VpxConfig->ts_layer_id[0] = 0;
					VpxConfig->ts_layer_id[1] = 2;
					VpxConfig->ts_layer_id[2] = 1;
					VpxConfig->ts_layer_id[3] = 2;
				}
				else
				{
					return EAVResult::Error;
				}

				if (bExternalRefControl)
				{
					VpxConfig->temporal_layering_mode = VP9E_TEMPORAL_LAYERING_MODE_BYPASS;
				}

				RefBuf.SetNum(8);

				if (InitAndSetControlSettings(PendingConfig) != EAVResult::Success)
				{
					return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to create TVideoEncoderLibVpxVP9"), TEXT("LibVpx"));
				}
			}
		}

		return TVideoEncoder<TResource, FVideoEncoderConfigLibVpx>::ApplyConfig();
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("LibVpx"));
}

template <typename TResource>
FAVResult TVideoEncoderLibVpxVP9<TResource>::SendFrame(TSharedPtr<FVideoResourceCPU> const& Resource, uint32 InTimestamp, bool bTriggerKeyFrame)
{
	if (!IsOpen())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("LibVpx"));
	}

	FAVResult AVResult = ApplyConfig();
	if (AVResult.IsNotSuccess())
	{
		return AVResult;
	}

	if (Resource.IsValid())
	{
		if (NumActiveSpatialLayers == 0)
		{
			// All spatial layers are disabled, return without encoding anything
			return EAVResult::Success;
		}

		if (bTriggerKeyFrame)
		{
			bForceKeyFrame = true;
		}

		if (PicsSinceKey + 1 == static_cast<size_t>(this->AppliedConfig.KeyframeInterval))
		{
			bForceKeyFrame = true;
		}

		if (SvcController)
		{
			LayerFrames = SvcController->NextFrameConfig(bForceKeyFrame);
			if (LayerFrames.IsEmpty())
			{
				return EAVResult::Error;
			}
			if (LayerFrames[0].GetIsKeyframe())
			{
				bForceKeyFrame = true;
			}
		}

		vpx_svc_layer_id_t LayerId = { 0 };
		if (!bForceKeyFrame)
		{
			const size_t GofIdx = (PicsSinceKey + 1) % Gof.NumFramesInGof;
			LayerId.temporal_layer_id = Gof.TemporalIdx[GofIdx];

			if (bForceAllActiveLayers)
			{
				LayerId.spatial_layer_id = FirstActiveLayer;
				bForceAllActiveLayers = false;
			}

			check(LayerId.spatial_layer_id <= NumActiveSpatialLayers);
			if (LayerId.spatial_layer_id >= NumActiveSpatialLayers)
			{
				// Drop entire picture.
				return EAVResult::Success;
			}
		}

		// Need to set temporal layer id on ALL layers, even disabled ones.
		// Otherwise libvpx might produce frames on a disabled layer:
		// http://crbug.com/1051476
		for (int sl_idx = 0; sl_idx < NumSpatialLayers; ++sl_idx)
		{
			LayerId.temporal_layer_id_per_spatial[sl_idx] = LayerId.temporal_layer_id;
		}

		if (LayerId.spatial_layer_id < FirstActiveLayer)
		{
			LayerId.spatial_layer_id = FirstActiveLayer;
		}

		if (SvcController)
		{
			LayerId.spatial_layer_id = LayerFrames[0].GetSpatialId();
			LayerId.temporal_layer_id = LayerFrames[0].GetTemporalId();
			for (const auto& Layer : LayerFrames)
			{
				LayerId.temporal_layer_id_per_spatial[Layer.GetSpatialId()] = Layer.GetTemporalId();
			}
			SetActiveSpatialLayers();
		}

		if (bIsSvc && PerformanceFlags.bUsePerLayerSpeed)
		{
			// Update speed settings that might depend on temporal index.
			bool bSpeedUpdated = false;
			for (int sl_idx = 0; sl_idx < NumSpatialLayers; ++sl_idx)
			{
				const int TargetSpeed =
					LayerId.temporal_layer_id_per_spatial[sl_idx] == 0
					? PerformanceFlagsBySpatialIndex[sl_idx].BaseLayerSpeed
					: PerformanceFlagsBySpatialIndex[sl_idx].HighLayerSpeed;
				if (SvcParams->speed_per_layer[sl_idx] != TargetSpeed)
				{
					SvcParams->speed_per_layer[sl_idx] = TargetSpeed;
					bSpeedUpdated = true;
				}
			}
			if (bSpeedUpdated)
			{
				SAFECONTROLVP9(Encoder.Get(), VP9E_SET_SVC_PARAMETERS, SvcParams.Get());
			}
		}

		SAFECONTROLVP9(Encoder.Get(), VP9E_SET_SVC_LAYER_ID, &LayerId);

		if (NumSpatialLayers > 1)
		{
			// Update frame dropping settings as they may change on per-frame basis.
			SAFECONTROLVP9(Encoder.Get(), VP9E_SET_SVC_FRAME_DROP_LAYER, SvcDropFrame.Get());
		}

		if (bVpxConfigChanged)
		{
			vpx_codec_err_t Result = ::vpx_codec_enc_config_set(Encoder.Get(), VpxConfig.Get());
			if (Result != VPX_CODEC_OK)
			{
				FString ErrorString(::vpx_codec_error_detail(Encoder.Get()));
				return FAVResult(EAVResult::Error, FString::Printf(TEXT("Error configuring encoder, error code: %d, details: %s"), Result, *ErrorString), TEXT("LibVpx"));
			}

			if (!PerformanceFlags.bUsePerLayerSpeed)
			{
				// Not setting individual speeds per layer, find the highest active
				// resolution instead and base the speed on that.
				for (int i = NumSpatialLayers - 1; i >= 0; --i)
				{
					if (VpxConfig->ss_target_bitrate[i] > 0)
					{
						int Width = (SvcParams->scaling_factor_num[i] * VpxConfig->g_w) / SvcParams->scaling_factor_den[i];
						int Height = (SvcParams->scaling_factor_num[i] * VpxConfig->g_h) / SvcParams->scaling_factor_den[i];

						TArray<int> ResolutionArray;
						PerformanceFlags.SettingsByResolution.GenerateKeyArray(ResolutionArray);
						size_t Index = Algo::LowerBound(ResolutionArray, Width * Height);
						if (Index > 0)
						{
							--Index;
						}

						int Speed = PerformanceFlags.SettingsByResolution[ResolutionArray[Index]].BaseLayerSpeed;

						SAFECONTROLVP9(Encoder.Get(), VP8E_SET_CPUUSED, Speed);
						break;
					}
				}
			}
			bVpxConfigChanged = false;
		}

		InputImage = MakeUnique<FInputImage>(InTimestamp);

		switch (Profile)
		{
		case EProfile::Profile0:
		{
			int StrideY = Resource->GetWidth();
			int StrideUV = (Resource->GetWidth() + 1) / 2;

			int DataSizeY = StrideY * Resource->GetHeight();
			int DataSizeUV = StrideUV * ((Resource->GetHeight() + 1) / 2);

			RawImage->planes[VPX_PLANE_Y] = static_cast<uint8*>(Resource->GetRaw().Get());
			RawImage->planes[VPX_PLANE_U] = static_cast<uint8*>(Resource->GetRaw().Get()) + DataSizeY;
			RawImage->planes[VPX_PLANE_V] = static_cast<uint8*>(Resource->GetRaw().Get()) + DataSizeY + DataSizeUV;
			RawImage->stride[VPX_PLANE_Y] = StrideY;
			RawImage->stride[VPX_PLANE_U] = StrideUV;
			RawImage->stride[VPX_PLANE_V] = StrideUV;

			break;
		}
		case EProfile::Profile1:
			checkNoEntry();
			break;
		case EProfile::Profile2:
		{
			int StrideY = Resource->GetWidth();
			int StrideUV = (Resource->GetWidth() + 1) / 2;

			int DataSizeY = 2 * StrideY * Resource->GetHeight();
			int DataSizeUV = 2 * StrideUV * ((Resource->GetHeight() + 1) / 2);

			RawImage->planes[VPX_PLANE_Y] = static_cast<uint8*>(Resource->GetRaw().Get());
			RawImage->planes[VPX_PLANE_U] = static_cast<uint8*>(Resource->GetRaw().Get()) + DataSizeY;
			RawImage->planes[VPX_PLANE_V] = static_cast<uint8*>(Resource->GetRaw().Get()) + DataSizeY + DataSizeUV;
			RawImage->stride[VPX_PLANE_Y] = StrideY * 2;
			RawImage->stride[VPX_PLANE_U] = StrideUV * 2;
			RawImage->stride[VPX_PLANE_V] = StrideUV * 2;
		}
		case EProfile::Profile3:
			checkNoEntry();
			break;
		}

		vpx_enc_frame_flags_t Flags = 0;
		if (bForceKeyFrame)
		{
			Flags = VPX_EFLAG_FORCE_KF;
		}

		if (SvcController)
		{
			vpx_svc_ref_frame_config_t RefConfig = SetReferences(LayerFrames);
			SAFECONTROLVP9(Encoder.Get(), VP9E_SET_SVC_REF_FRAME_CONFIG, &RefConfig);
		}
		else if (bExternalRefControl)
		{
			vpx_svc_ref_frame_config_t RefConfig = SetReferences(bForceKeyFrame, LayerId.spatial_layer_id);
			SAFECONTROLVP9(Encoder.Get(), VP9E_SET_SVC_REF_FRAME_CONFIG, &RefConfig);
		}

		bFirstFrameInPicture = true;

		// TODO(ssilkin): Frame duration should be specified per spatial layer
		// since their frame rate can be different. For now calculate frame duration
		// based on target frame rate of the highest spatial layer, which frame rate
		// is supposed to be equal or higher than frame rate of low spatial layers.
		// Also, timestamp should represent actual time passed since previous frame
		// (not 'expected' time). Then rate controller can drain buffer more
		// accurately.
		uint32_t Duration = static_cast<uint32_t>(90000 / this->AppliedConfig.Framerate);

		const vpx_codec_err_t Result = ::vpx_codec_encode(Encoder.Get(), RawImage.Get(), Timestamp, Duration, Flags, VPX_DL_REALTIME);
		if (Result != VPX_CODEC_OK)
		{
			FString ErrorString(::vpx_codec_error_detail(Encoder.Get()));
			return FAVResult(EAVResult::Error, FString::Printf(TEXT("Error encoding, error code: %d, details: %s"), Result, *ErrorString), TEXT("LibVpx"));
		}
		Timestamp += Duration;

		if (bLayerBuffering)
		{
			const bool bEndOfPicture = true;
			DeliverBufferedFrame(bEndOfPicture);
		}

		return EAVResult::Success;
	}
	else
	{
		// Flush encoder
		return EAVResult::Success;
	}
}

template <typename TResource>
FAVResult TVideoEncoderLibVpxVP9<TResource>::ReceivePacket(FVideoPacket& OutPacket)
{
	if (IsOpen())
	{
		if (Packets.Dequeue(OutPacket))
		{
			return EAVResult::Success;
		}

		return EAVResult::PendingInput;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("LibVpx"));
}

template <typename TResource>
FAVResult TVideoEncoderLibVpxVP9<TResource>::Destroy()
{
	EAVResult Result = EAVResult::Success;

	if (Encoder.IsValid())
	{
		Encoder.Reset();
	}

	if (RawImage.IsValid())
	{
		RawImage.Reset();
	}

	if (VpxConfig.IsValid())
	{
		VpxConfig.Reset();
	}

	if (SvcParams.IsValid())
	{
		SvcParams.Reset();
	}

	return Result;
}

template <typename TResource>
uint32_t TVideoEncoderLibVpxVP9<TResource>::MaxIntraTarget(uint32_t OptimalBuffersize, uint32_t MaxFramerate)
{
	// Set max to the optimal buffer level (normalized by target BR),
	// and scaled by a scale_par.
	// Max target size = scale_par * optimal_buffer_size * targetBR[Kbps].
	// This value is presented in percentage of perFrameBw:
	// perFrameBw = targetBR[Kbps] * 1000 / framerate.
	// The target in % is as follows:
	float	 ScalePar = 0.5;
	uint32_t TargetPct = OptimalBuffersize * ScalePar * MaxFramerate / 10;

	// Don't go below 3 times the per frame bandwidth.
	const uint32_t MinIntraTh = 300;
	return (TargetPct < MinIntraTh) ? MinIntraTh : TargetPct;
}

template <typename TResource>
int TVideoEncoderLibVpxVP9<TResource>::NumberOfThreads(uint32_t Width, uint32_t Height, int Cpus)
{
	// Keep the number of encoder threads equal to the possible number of column
	// tiles, which is (1, 2, 4, 8). See comments below for VP9E_SET_TILE_COLUMNS.
	if (Width * Height >= 1280 * 720 && Cpus > 8)
	{
		return 8;
	}
	else if (Width * Height >= 1280 * 720 && Cpus > 4)
	{
		return 4;
	}
	else if (Width * Height >= 640 * 360 && Cpus > 2)
	{
		return 2;
	}
	else
	{
		return 1;
	}
}

template <typename TResource>
FAVResult TVideoEncoderLibVpxVP9<TResource>::InitAndSetControlSettings(FVideoEncoderConfigLibVpx const& Config)
{
	// Set QP-min/max per spatial and temporal Layer.
	int TotalNumLayers = NumSpatialLayers * NumTemporalLayers;
	for (int i = 0; i < TotalNumLayers; ++i)
	{
		SvcParams->max_quantizers[i] = VpxConfig->rc_max_quantizer;
		SvcParams->min_quantizers[i] = VpxConfig->rc_min_quantizer;
	}
	VpxConfig->ss_number_layers = NumSpatialLayers;
	if (SvcController)
	{
		FScalableVideoController::FStreamLayersConfig StreamConfig = SvcController->StreamConfig();
		for (int i = 0; i < StreamConfig.NumSpatialLayers; ++i)
		{
			SvcParams->scaling_factor_num[i] = StreamConfig.ScalingFactors[i].Num;
			SvcParams->scaling_factor_den[i] = StreamConfig.ScalingFactors[i].Den;
		}
	}
	else if (ExplicitlyConfiguredSpatialLayers(Config))
	{
		for (int i = 0; i < NumSpatialLayers; ++i)
		{
			const FSpatialLayer& Layer = Config.SpatialLayers[i];
			check(Layer.Width > 0);
			const int ScaleFactor = Config.Width / Layer.Width;
			check(ScaleFactor > 0);

			// Ensure scaler factor is integer.
			if (ScaleFactor * Layer.Width != Config.Width)
			{
				return EAVResult::Error;
			}

			// Ensure scale factor is the same in both dimensions.
			if (ScaleFactor * Layer.Height != Config.Height)
			{
				return EAVResult::Error;
			}

			// Ensure scale factor is power of two.
			const bool bIsPowOfTwo = (ScaleFactor & (ScaleFactor - 1)) == 0;
			if (!bIsPowOfTwo)
			{
				return EAVResult::Error;
			}

			SvcParams->scaling_factor_num[i] = 1;
			SvcParams->scaling_factor_den[i] = ScaleFactor;

			check(Config.SpatialLayers[i].Framerate > 0);
			check(Config.SpatialLayers[i].Framerate <= Config.Framerate);
			if (i > 0)
			{
				// Frame rate of high spatial Layer is supposed to be equal or higher than frame rate of low spatial Layer.
				check(Config.SpatialLayers[i].Framerate >= Config.SpatialLayers[i - 1].Framerate);
			}
		}
	}
	else
	{
		for (int i = NumSpatialLayers - 1; i >= 0; --i)
		{
			// 1:2 scaling in each dimension.
			SvcParams->scaling_factor_num[i] = 128;
			SvcParams->scaling_factor_den[i] = 256;
		}
	}

	UpdatePerformanceFlags(Config);
	check(PerformanceFlagsBySpatialIndex.Num() == static_cast<size_t>(NumSpatialLayers));

	FVideoEncoderConfig BaseConfig;
	FAVExtension::TransformConfig(BaseConfig, Config);

	FVideoBitrateAllocatorSVC InitAllocator(BaseConfig);
	CurrentBitrateAllocation = InitAllocator.Allocate(FVideoBitrateAllocationParameters(Config.MinBitrate, FFrameRate(Config.Framerate, 1)));
	if (!SetSvcRates(Config, CurrentBitrateAllocation))
	{
		return EAVResult::Error;
	}

	const vpx_codec_err_t Result = ::vpx_codec_enc_init(Encoder.Get(), vpx_codec_vp9_cx(), VpxConfig.Get(), VpxConfig->g_bit_depth == VPX_BITS_8 ? 0 : VPX_CODEC_USE_HIGHBITDEPTH);
	if (Result != VPX_CODEC_OK)
	{
		return FAVResult(EAVResult::Error, FString::Printf(TEXT("Init error %hs"), ::vpx_codec_err_to_string(Result)), TEXT("LibVpx"));
	}

	if (PerformanceFlags.bUsePerLayerSpeed)
	{
		for (int si = 0; si < NumSpatialLayers; ++si)
		{
			SvcParams->speed_per_layer[si] = PerformanceFlagsBySpatialIndex[si].BaseLayerSpeed;
			SvcParams->loopfilter_ctrl[si] = PerformanceFlagsBySpatialIndex[si].DeblockMode;
		}
		bool bDenoiserOn = Config.bDenoisingOn && PerformanceFlagsBySpatialIndex[NumSpatialLayers - 1].bAllowDenoising;
		::vpx_codec_control(Encoder.Get(), VP9E_SET_NOISE_SENSITIVITY, bDenoiserOn ? 1 : 0);
	}

	SAFECONTROLVP9(Encoder.Get(), VP8E_SET_MAX_INTRA_BITRATE_PCT, RCMaxIntraTarget);
	SAFECONTROLVP9(Encoder.Get(), VP9E_SET_AQ_MODE, Config.bAdaptiveQpMode ? 3 : 0);

	SAFECONTROLVP9(Encoder.Get(), VP9E_SET_FRAME_PARALLEL_DECODING, 0);
	SAFECONTROLVP9(Encoder.Get(), VP9E_SET_SVC_GF_TEMPORAL_REF, 0);

	if (bIsSvc)
	{
		SAFECONTROLVP9(Encoder.Get(), VP9E_SET_SVC, 1);
		SAFECONTROLVP9(Encoder.Get(), VP9E_SET_SVC_PARAMETERS, SvcParams.Get());
	}
	if (!bIsSvc || !PerformanceFlags.bUsePerLayerSpeed)
	{
		SAFECONTROLVP9(Encoder.Get(), VP8E_SET_CPUUSED, PerformanceFlagsBySpatialIndex.Last().BaseLayerSpeed);
	}

	if (NumSpatialLayers > 1)
	{
		switch (InterLayerPrediction)
		{
		case EInterLayerPrediction::On:
			SAFECONTROLVP9(Encoder.Get(), VP9E_SET_SVC_INTER_LAYER_PRED, 0);
			break;
		case EInterLayerPrediction::Off:
			SAFECONTROLVP9(Encoder.Get(), VP9E_SET_SVC_INTER_LAYER_PRED, 1);
			break;
		case EInterLayerPrediction::OnKeyPicture:
			SAFECONTROLVP9(Encoder.Get(), VP9E_SET_SVC_INTER_LAYER_PRED, 2);
			break;
		default:
			checkNoEntry();
		}

		if (!SvcDropFrame.IsValid())
		{
			SvcDropFrame.Reset(new vpx_svc_frame_drop_t);
		}

		memset(SvcDropFrame.Get(), 0, sizeof(vpx_svc_frame_drop_t));

		// Configure encoder to drop entire superframe whenever it needs to drop
		// a layer. This mode is preferred over per-layer dropping which causes
		// quality flickering and is not compatible with RTP non-flexible mode.
		SvcDropFrame->framedrop_mode = bFullSuperframeDrop ? FULL_SUPERFRAME_DROP : CONSTRAINED_LAYER_DROP;
		// Buffering is needed only for constrained layer drop, as it's not clear
		// which frame is the last.
		bLayerBuffering = !bFullSuperframeDrop;
		SvcDropFrame->max_consec_drop = TNumericLimits<int>::Max();
		for (size_t i = 0; i < NumSpatialLayers; ++i)
		{
			SvcDropFrame->framedrop_thresh[i] = VpxConfig->rc_dropframe_thresh;
		}

		SAFECONTROLVP9(Encoder.Get(), VP9E_SET_SVC_FRAME_DROP_LAYER, SvcDropFrame.Get());
	}

	// Register callback for getting each spatial layer.
	vpx_codec_priv_output_cx_pkt_cb_pair_t cbp = {
		Internal::EncoderOutputCodedPacketCallback<TResource>,
		reinterpret_cast<void*>(this)
	};

	SAFECONTROLVP9(Encoder.Get(), VP9E_REGISTER_CX_CALLBACK, reinterpret_cast<void*>(&cbp));

	// Control function to set the number of column tiles in encoding a frame, in
	// log2 unit: e.g., 0 = 1 tile column, 1 = 2 tile columns, 2 = 4 tile columns.
	// The number tile columns will be capped by the encoder based on image size
	// (minimum width of tile column is 256 pixels, maximum is 4096).
	SAFECONTROLVP9(Encoder.Get(), VP9E_SET_TILE_COLUMNS, static_cast<int>((VpxConfig->g_threads >> 1)));

	// Turn on row-based multithreading.
	SAFECONTROLVP9(Encoder.Get(), VP9E_SET_ROW_MT, 1);

	if (!PerformanceFlags.bUsePerLayerSpeed)
	{
		::vpx_codec_control(Encoder.Get(), VP9E_SET_NOISE_SENSITIVITY, Config.bDenoisingOn ? 1 : 0);
	}

	// Enable encoder skip of static/low content blocks.
	SAFECONTROLVP9(Encoder.Get(), VP8E_SET_STATIC_THRESHOLD, 1);

	bVpxConfigChanged = true;

	return EAVResult::Success;
}

template <typename TResource>
TUniquePtr<FScalableVideoController> TVideoEncoderLibVpxVP9<TResource>::CreateScalabilityStructureFromConfig(FVideoEncoderConfigLibVpx const& Config)
{
	int NumSpatialLayer = Config.NumberOfSpatialLayers;
	int NumTemporalLayer = FMath::Max(1, int{ Config.NumberOfTemporalLayers });
	if (NumSpatialLayer == 1 && NumTemporalLayer == 1)
	{
		return MakeUnique<FScalableVideoControllerNoLayering>();
	}

	TStringBuilder<64> Builder;
	if (Config.InterLayerPrediction == EInterLayerPrediction::On || NumSpatialLayer == 1)
	{
		Builder << TEXT("L") << NumSpatialLayer << TEXT("T") << NumTemporalLayer;
	}
	else if (Config.InterLayerPrediction == EInterLayerPrediction::OnKeyPicture)
	{
		Builder << TEXT("L") << NumSpatialLayer << TEXT("T") << NumTemporalLayer << TEXT("_KEY");
	}
	else
	{
		check(Config.InterLayerPrediction == EInterLayerPrediction::Off);
		Builder << TEXT("S") << NumSpatialLayer << TEXT("T") << NumTemporalLayer;
	}

	// Check spatial ratio.
	if (NumSpatialLayer > 1 && Config.SpatialLayers[0].TargetBitrate > 0)
	{
		if (Config.Width != Config.SpatialLayers[NumSpatialLayer - 1].Width || Config.Height != Config.SpatialLayers[NumSpatialLayer - 1].Height)
		{
			FAVResult::Log(EAVResult::Warning, TEXT("Top Layer resolution expected to match overall resolution"), TEXT("LibVpx"));
			return nullptr;
		}
		// Check if the ratio is one of the supported.
		int Numerator;
		int Denominator;
		if (Config.SpatialLayers[1].Width == 2 * Config.SpatialLayers[0].Width)
		{
			Numerator = 1;
			Denominator = 2;
			// no suffix for 1:2 ratio.
		}
		else if (2 * Config.SpatialLayers[1].Width == 3 * Config.SpatialLayers[0].Width)
		{
			Numerator = 2;
			Denominator = 3;
			Builder << TEXT("h");
		}
		else
		{
			FAVResult::Log(EAVResult::Warning, FString::Printf(TEXT("Unsupported scalability ratio %d:%d"), Config.SpatialLayers[0].Width, Config.SpatialLayers[1].Width), TEXT("LibVpx"));
			return nullptr;
		}
		// Validate ratio is consistent for all spatial Layer transitions.
		for (int Sid = 1; Sid < NumSpatialLayer; ++Sid)
		{
			if (Config.SpatialLayers[Sid].Width * Numerator != Config.SpatialLayers[Sid - 1].Width * Denominator || Config.SpatialLayers[Sid].Height * Numerator != Config.SpatialLayers[Sid - 1].Height * Denominator)
			{
				FAVResult::Log(EAVResult::Warning, FString::Printf(TEXT("Unsupported scalability ratio %d:%d"), Numerator, Denominator), TEXT("LibVpx"));
				return nullptr;
			}
		}
	}

	TOptional<EScalabilityMode> ScalabilityMode = ScalabilityModeFromString(FString(Builder));
	if (!ScalabilityMode.IsSet())
	{
		FAVResult::Log(EAVResult::Warning, FString::Printf(TEXT("Invalid scalability mode %s"), Builder.ToString()), TEXT("LibVpx"));
		return nullptr;
	}

	TUniquePtr<FScalableVideoController> ScalabilityStructureController = CreateScalabilityStructure(*ScalabilityMode);
	if (!ScalabilityStructureController)
	{
		FAVResult::Log(EAVResult::Warning, FString::Printf(TEXT("Unsupported scalability structure %s"), Builder.ToString()), TEXT("LibVpx"));
	}
	else
	{
		FAVResult::Log(EAVResult::Success, FString::Printf(TEXT("Created scalability structure %s"), Builder.ToString()), TEXT("LibVpx"));
	}
	return ScalabilityStructureController;
}

template <typename TResource>
void TVideoEncoderLibVpxVP9<TResource>::EnableSpatialLayer(int Sid)
{
	check(Sid < NumSpatialLayers);
	if (VpxConfig->ss_target_bitrate[Sid] > 0)
	{
		return;
	}
	for (int Tid = 0; Tid < NumTemporalLayers; ++Tid)
	{
		VpxConfig->layer_target_bitrate[Sid * NumTemporalLayers + Tid] = CurrentBitrateAllocation.GetTemporalLayerSumBitrate(Sid, Tid) / 1000;
	}
	VpxConfig->ss_target_bitrate[Sid] = CurrentBitrateAllocation.GetSpatialLayerSumBitrate(Sid) / 1000;
	check(VpxConfig->ss_target_bitrate[Sid] > 0);
	bVpxConfigChanged = true;
}

template <typename TResource>
void TVideoEncoderLibVpxVP9<TResource>::DisableSpatialLayer(int Sid)
{
	check(Sid < NumSpatialLayers);
	if (VpxConfig->ss_target_bitrate[Sid] == 0)
	{
		return;
	}
	VpxConfig->ss_target_bitrate[Sid] = 0;
	for (int Tid = 0; Tid < NumTemporalLayers; ++Tid)
	{
		VpxConfig->layer_target_bitrate[Sid * NumTemporalLayers + Tid] = 0;
	}
	bVpxConfigChanged = true;
}

template <typename TResource>
void TVideoEncoderLibVpxVP9<TResource>::SetActiveSpatialLayers()
{
	// Svc controller may decide to skip a frame at certain spatial layer even
	// when bitrate for it is non-zero, however libvpx uses configured bitrate as
	// a signal which layers should be produced.
	check(SvcController);
	check(!LayerFrames.IsEmpty());

	auto FrameIt = LayerFrames.CreateIterator();
	for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
	{
		if (FrameIt && FrameIt->GetSpatialId() == Sid)
		{
			EnableSpatialLayer(Sid);
			++FrameIt;
		}
		else
		{
			DisableSpatialLayer(Sid);
		}
	}
}

template <typename TResource>
void TVideoEncoderLibVpxVP9<TResource>::GetEncodedLayerFrame(const vpx_codec_cx_pkt* Packet)
{
	check(Packet->kind == VPX_CODEC_CX_FRAME_PKT);

	if (Packet->data.frame.sz == 0)
	{
		// Ignore dropped frame.
		return;
	}

	vpx_svc_layer_id_t LayerId = { 0 };
	SAFECONTROLVP9(Encoder.Get(), VP9E_GET_SVC_LAYER_ID, &LayerId);

	if (bLayerBuffering)
	{
		// Deliver buffered low spatial layer frame.
		const bool bEndOfPicture = false;
		DeliverBufferedFrame(bEndOfPicture);
	}

	TArray<uint8> EncodedData(static_cast<uint8*>(Packet->data.frame.buf), Packet->data.frame.sz);
	EncodedImage.SetEncodedData(EncodedData);

	CodecSpecific = {};
	TOptional<int> SpatialIndex;
	TOptional<int> TemporalIndex;
	if (!PopulateCodecSpecific(&CodecSpecific, &SpatialIndex, &TemporalIndex, *Packet))
	{
		// Drop the frame.
		EncodedImage.SetSize(0);
		return;
	}
	EncodedImage.SpatialIndex = SpatialIndex;
	EncodedImage.TemporalIndex = TemporalIndex;

	const bool bIsKeyFrame = ((Packet->data.frame.flags & VPX_FRAME_IS_KEY) ? true : false) && !CodecSpecific.CodecSpecific.VP9.bInterLayerPredicted;

	// Ensure encoder issued key frame on request.
	check(bIsKeyFrame || !bForceKeyFrame);

	// Check if encoded frame is a key frame.
	EncodedImage.FrameType = EFrameType::P;
	if (bIsKeyFrame)
	{
		EncodedImage.FrameType = EFrameType::I;
		bForceKeyFrame = false;
	}

	UpdateReferenceBuffers(*Packet, PicsSinceKey);

	EncodedImage.Timestamp = InputImage->Timestamp;
	EncodedImage.Width = Packet->data.frame.height[LayerId.spatial_layer_id];
	EncodedImage.Height = Packet->data.frame.width[LayerId.spatial_layer_id];
	int QP = -1;
	SAFECONTROLVP9(Encoder.Get(), VP8E_GET_LAST_QUANTIZER, &QP);
	EncodedImage.QP = QP;

	if (!bLayerBuffering)
	{
		const bool bEndOfPicture = EncodedImage.SpatialIndex.Get(0) + 1 == NumActiveSpatialLayers;
		DeliverBufferedFrame(bEndOfPicture);
	}
}

template <typename TResource>
void TVideoEncoderLibVpxVP9<TResource>::DeliverBufferedFrame(bool bEndOfPicture)
{
	if (EncodedImage.GetSize() > 0)
	{
		if (NumSpatialLayers > 1)
		{
			// Restore frame dropping settings, as dropping may be temporary forbidden
			// due to dynamically enabled layers.
			for (size_t i = 0; i < NumSpatialLayers; ++i)
			{
				SvcDropFrame->framedrop_thresh[i] = VpxConfig->rc_dropframe_thresh;
			}
		}

		CodecSpecific.bEndOfPicture = bEndOfPicture;

		TSharedPtr<uint8> const CopiedData = MakeShareable(new uint8[EncodedImage.GetSize()]);
		FMemory::BigBlockMemcpy(CopiedData.Get(), EncodedImage.GetEncodedData().GetData(), EncodedImage.GetSize());

		FVideoPacket Packet = FVideoPacket(CopiedData, EncodedImage.GetSize(), EncodedImage.Timestamp, ++FrameCount, EncodedImage.QP, EncodedImage.FrameType == EFrameType::I);
		Packet.CodecSpecificInfo = CodecSpecific;
		Packet.SpatialIndex = EncodedImage.SpatialIndex;
		Packet.TemporalIndex = EncodedImage.TemporalIndex;
		Packets.Enqueue(Packet);

		EncodedImage.SetSize(0);
	}
}

template <typename TResource>
bool TVideoEncoderLibVpxVP9<TResource>::PopulateCodecSpecific(FCodecSpecificInfo* CodecSpecificInfo, TOptional<int>* SpatialIdx, TOptional<int>* TemporalIdx, const vpx_codec_cx_pkt& Packet)
{
	CodecSpecificInfo->Codec = EVideoCodec::VP9;

	FCodecSpecificInfoVP9& Info = CodecSpecificInfo->CodecSpecific.VP9;

	Info.bFirstFrameInPicture = bFirstFrameInPicture;
	Info.bFlexibleMode = bIsFlexibleMode;

	if (Packet.data.frame.flags & VPX_FRAME_IS_KEY)
	{
		PicsSinceKey = 0;
	}
	else if (bFirstFrameInPicture)
	{
		PicsSinceKey++;
	}

	vpx_svc_layer_id_t LayerId = { 0 };
	SAFECONTROLVP9(Encoder.Get(), VP9E_GET_SVC_LAYER_ID, &LayerId);

	// Can't have keyframe with non-zero temporal layer.
	check(PicsSinceKey != 0 || LayerId.temporal_layer_id == 0);

	check(NumTemporalLayers > 0);
	check(NumActiveSpatialLayers > 0);
	if (NumTemporalLayers == 1)
	{
		check(LayerId.temporal_layer_id == 0);
		Info.TemporalIdx = 0xFF;
		*TemporalIdx = FNullOpt(0);
	}
	else
	{
		Info.TemporalIdx = LayerId.temporal_layer_id;
		*TemporalIdx = LayerId.temporal_layer_id;
	}
	if (NumActiveSpatialLayers == 1)
	{
		check(LayerId.spatial_layer_id == 0);
		*SpatialIdx = FNullOpt(0);
	}
	else
	{
		*SpatialIdx = LayerId.spatial_layer_id;
	}

	const bool bIsKeyPic = (PicsSinceKey == 0);
	const bool bIsInterLayerPredAllowed = (InterLayerPrediction == EInterLayerPrediction::On || (InterLayerPrediction == EInterLayerPrediction::OnKeyPicture && bIsKeyPic));

	// Always set _interLayerPredicted to true on high layer frame if inter-layer
	// prediction (ILP) is allowed even if encoder didn't actually use it.
	// Setting _interLayerPredicted to false would allow receiver to decode high
	// layer frame without decoding low layer frame. If that would happen (e.g.
	// if low layer frame is lost) then receiver won't be able to decode next high
	// layer frame which uses ILP.
	Info.bInterLayerPredicted = bFirstFrameInPicture ? false : bIsInterLayerPredAllowed;

	// Mark all low spatial layer frames as references (not just frames of
	// active low spatial layers) if inter-layer prediction is enabled since
	// these frames are indirect references of high spatial layer, which can
	// later be enabled without key frame.
	Info.bNonRefForInterLayerPred = !bIsInterLayerPredAllowed || LayerId.spatial_layer_id + 1 == NumSpatialLayers;

	// Always populate this, so that the packetizer can properly set the marker
	// bit.
	Info.NumSpatialLayers = NumActiveSpatialLayers;
	Info.FirstActiveLayer = FirstActiveLayer;

	Info.NumRefPics = 0;
	FillReferenceIndices(Packet, PicsSinceKey, Info.bInterLayerPredicted, &Info);
	if (Info.bFlexibleMode)
	{
		Info.GofIdx = 0xFF;
		if (!SvcController)
		{
			if (NumTemporalLayers == 1)
			{
				Info.bTemporalUpSwitch = true;
			}
			else
			{
				// In flexible mode with > 1 temporal layer but no SVC controller we
				// can't techincally determine if a frame is an upswitch point, use
				// gof-based data as proxy for now.
				// TODO(sprang): Remove once SVC controller is the only choice.
				Info.GofIdx = static_cast<uint8_t>(PicsSinceKey % Gof.NumFramesInGof);
				Info.bTemporalUpSwitch = Gof.TemporalUpSwitch[Info.GofIdx];
			}
		}
	}
	else
	{
		Info.GofIdx = static_cast<uint8_t>(PicsSinceKey % Gof.NumFramesInGof);
		Info.bTemporalUpSwitch = Gof.TemporalUpSwitch[Info.GofIdx];
		check(Info.NumRefPics == Gof.NumRefPics[Info.GofIdx] || Info.NumRefPics == 0);
	}

	Info.bInterPicPredicted = (!bIsKeyPic && Info.NumRefPics > 0);

	// Write SS on key frame of independently coded spatial layers and on base
	// temporal/spatial layer frame if number of layers changed without issuing
	// of key picture (inter-layer prediction is enabled).
	const bool bIsKeyFrame = bIsKeyPic && !Info.bInterLayerPredicted;
	if (bIsKeyFrame || (bSsInfoNeeded && LayerId.temporal_layer_id == 0 && LayerId.spatial_layer_id == FirstActiveLayer))
	{
		Info.bSSDataAvailable = true;
		Info.bSpatialLayerResolutionPresent = true;
		// Signal disabled layers.
		for (size_t i = 0; i < FirstActiveLayer; ++i)
		{
			Info.Width[i] = 0;
			Info.Height[i] = 0;
		}
		for (size_t i = FirstActiveLayer; i < NumActiveSpatialLayers; ++i)
		{
			Info.Width[i] = this->AppliedConfig.Width * SvcParams->scaling_factor_num[i] / SvcParams->scaling_factor_den[i];
			Info.Height[i] = this->AppliedConfig.Height * SvcParams->scaling_factor_num[i] / SvcParams->scaling_factor_den[i];
		}
		if (Info.bFlexibleMode)
		{
			Info.Gof.NumFramesInGof = 0;
		}
		else
		{
			Info.Gof.NumFramesInGof = Gof.NumFramesInGof;
			for (size_t i = 0; i < Gof.NumFramesInGof; ++i)
			{
				Info.Gof.TemporalIdx[i] = Gof.TemporalIdx[i];
				Info.Gof.TemporalUpSwitch[i] = Gof.TemporalUpSwitch[i];
				Info.Gof.NumRefPics[i] = Gof.NumRefPics[i];
				for (uint8_t r = 0; r < Gof.NumRefPics[i]; ++r)
				{
					Info.Gof.PidDiff[i][r] = Gof.PidDiff[i][r];
				}
			}
		}

		bSsInfoNeeded = false;
	}
	else
	{
		Info.bSSDataAvailable = false;
	}

	bFirstFrameInPicture = false;

	// Populate codec-agnostic section in the codec specific structure
	if (SvcController)
	{
		FScalableVideoController::FLayerFrameConfig* FoundLayer = LayerFrames.FindByPredicate([&LayerId](FScalableVideoController::FLayerFrameConfig& Config) {
			return Config.GetSpatialId() == LayerId.spatial_layer_id;
			});
		if (FoundLayer == nullptr)
		{
			// UE_LOGFMT(LogPixelStreamingEpicRtc, Error, "Encoder produced a frame for layer S{0}T{1} that wasn't requested", LayerId.spatial_layer_id, LayerId.temporal_layer_id);
			return false;
		}

		CodecSpecificInfo->GenericFrameInfo = SvcController->OnEncodeDone(*FoundLayer);
		if (bIsKeyFrame)
		{
			CodecSpecificInfo->TemplateStructure = SvcController->DependencyStructure();
			TArray<FIntPoint>& Resolutions = (*(CodecSpecificInfo->TemplateStructure)).Resolutions;
			Resolutions.SetNum(NumSpatialLayers);
			for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
			{
				Resolutions[Sid] = FIntPoint(
					static_cast<int32_t>(this->AppliedConfig.Width * SvcParams->scaling_factor_num[Sid] / SvcParams->scaling_factor_den[Sid]),
					static_cast<int32_t>(this->AppliedConfig.Height * SvcParams->scaling_factor_num[Sid] / SvcParams->scaling_factor_den[Sid]));
			}
		}
		if (bIsFlexibleMode)
		{
			// Populate data for legacy temporal-upswitch state.
			// We can switch up to a higher temporal layer only if all temporal layers
			// higher than this (within the current spatial layer) are switch points.
			Info.bTemporalUpSwitch = true;
			for (int i = LayerId.temporal_layer_id + 1; i < NumTemporalLayers; ++i)
			{
				// Assumes decode targets are always ordered first by spatial then by
				// temporal id.
				size_t dti_index = (LayerId.spatial_layer_id * NumTemporalLayers) + i;
				Info.bTemporalUpSwitch &= ((*(CodecSpecificInfo->GenericFrameInfo)).DecodeTargetIndications[dti_index] == EDecodeTargetIndication::Switch);
			}
		}
	}
	return true;
}

template <typename TResource>
void TVideoEncoderLibVpxVP9<TResource>::FillReferenceIndices(const vpx_codec_cx_pkt& Packet, const size_t PicNum, const bool bInterLayerPredicted, FCodecSpecificInfoVP9* Info)
{
	vpx_svc_layer_id_t LayerId = { 0 };
	SAFECONTROLVP9(Encoder.Get(), VP9E_GET_SVC_LAYER_ID, &LayerId);

	const bool bIsKeyFrame = (Packet.data.frame.flags & VPX_FRAME_IS_KEY) ? true : false;

	TArray<FRefFrameBuffer> RefBufList;

	if (bIsSvc)
	{
		vpx_svc_ref_frame_config_t EncLayerConf = { { 0 } };
		SAFECONTROLVP9(Encoder.Get(), VP9E_GET_SVC_REF_FRAME_CONFIG, &EncLayerConf);
		char RefBufFlags[] = "00000000";
		// There should be one character per buffer + 1 termination '\0'.
		static_assert(sizeof(RefBufFlags) == 9);

		if (EncLayerConf.reference_last[LayerId.spatial_layer_id])
		{
			const size_t fb_idx = EncLayerConf.lst_fb_idx[LayerId.spatial_layer_id];
			check(fb_idx < RefBuf.Num());
			if (RefBufList.Find(RefBuf[fb_idx]) == INDEX_NONE)
			{
				RefBufList.Add(RefBuf[fb_idx]);
				RefBufFlags[fb_idx] = '1';
			}
		}

		if (EncLayerConf.reference_alt_ref[LayerId.spatial_layer_id])
		{
			const size_t fb_idx = EncLayerConf.alt_fb_idx[LayerId.spatial_layer_id];
			check(fb_idx < RefBuf.Num());
			if (RefBufList.Find(RefBuf[fb_idx]) == INDEX_NONE)
			{
				RefBufList.Add(RefBuf[fb_idx]);
				RefBufFlags[fb_idx] = '1';
			}
		}

		if (EncLayerConf.reference_golden[LayerId.spatial_layer_id])
		{
			const size_t fb_idx = EncLayerConf.gld_fb_idx[LayerId.spatial_layer_id];
			check(fb_idx < RefBuf.Num());
			if (RefBufList.Find(RefBuf[fb_idx]) == INDEX_NONE)
			{
				RefBufList.Add(RefBuf[fb_idx]);
				RefBufFlags[fb_idx] = '1';
			}
		}

		FAVResult::Log(EAVResult::Success, FString::Printf(TEXT("Frame %zu S%dT%d referenced buffers %hs"), PicNum, LayerId.spatial_layer_id, LayerId.temporal_layer_id, RefBufFlags), TEXT("LibVpx"));
	}
	else if (!bIsKeyFrame)
	{
		check(NumSpatialLayers == 1);
		check(NumTemporalLayers == 1);
		// In non-SVC mode encoder doesn't provide reference list. Assume each frame
		// refers previous one, which is stored in buffer 0.
		RefBufList.Add(RefBuf[0]);
	}

	TArray<size_t> RefPidList;

	Info->NumRefPics = 0;
	for (const FRefFrameBuffer& Ref_Buf : RefBufList)
	{
		check(Ref_Buf.PicNum <= PicNum);
		if (Ref_Buf.PicNum < PicNum)
		{
			if (InterLayerPrediction != EInterLayerPrediction::On)
			{
				// RTP spec limits temporal prediction to the same spatial layer.
				// It is safe to ignore this requirement if inter-layer prediction is
				// enabled for all frames when all base frames are relayed to receiver.
				check(Ref_Buf.SpatialLayerId == LayerId.spatial_layer_id);
			}
			else
			{
				check(Ref_Buf.SpatialLayerId <= LayerId.spatial_layer_id);
			}
			check(Ref_Buf.TemporalLayerId <= LayerId.temporal_layer_id);

			// Encoder may reference several spatial layers on the same previous
			// frame in case if some spatial layers are skipped on the current frame.
			// We shouldn't put duplicate references as it may break some old
			// clients and isn't RTP compatible.
			if (RefPidList.Find(Ref_Buf.PicNum) != INDEX_NONE)
			{
				continue;
			}
			RefPidList.Add(Ref_Buf.PicNum);

			const size_t PDiff = PicNum - Ref_Buf.PicNum;
			check(PDiff <= 127UL);

			Info->PDiff[Info->NumRefPics] = static_cast<uint8_t>(PDiff);
			++Info->NumRefPics;
		}
		else
		{
			check(bInterLayerPredicted);
			// RTP spec only allows to use previous spatial layer for inter-layer
			// prediction.
			check(Ref_Buf.SpatialLayerId + 1 == LayerId.spatial_layer_id);
		}
	}
}

template <typename TResource>
void TVideoEncoderLibVpxVP9<TResource>::UpdateReferenceBuffers(const vpx_codec_cx_pkt& Packet, size_t PicNum)
{
	vpx_svc_layer_id_t LayerId = { 0 };
	SAFECONTROLVP9(Encoder.Get(), VP9E_GET_SVC_LAYER_ID, &LayerId)

		FRefFrameBuffer FrameBuf = {
			.PicNum = PicNum,
			.SpatialLayerId = LayerId.spatial_layer_id,
			.TemporalLayerId = LayerId.temporal_layer_id
	};

	if (bIsSvc)
	{
		vpx_svc_ref_frame_config_t EncLayerConf = { { 0 } };
		SAFECONTROLVP9(Encoder.Get(), VP9E_GET_SVC_REF_FRAME_CONFIG, &EncLayerConf);
		const int bUpdateBufferSlot = EncLayerConf.update_buffer_slot[LayerId.spatial_layer_id];

		TStaticBitArray<8> BitArray;

		for (size_t i = 0; i < RefBuf.Num(); ++i)
		{
			if (bUpdateBufferSlot & (1 << i))
			{
				RefBuf[i] = FrameBuf;
				BitArray[7 - i] = true;
			}
		}

		FAVResult::Log(EAVResult::Success, FString::Printf(TEXT("Frame %zu S%dT%d updated buffers %s"), PicNum, LayerId.spatial_layer_id, LayerId.temporal_layer_id, *BitArray.ToString()), TEXT("LibVpx"));
	}
	else
	{
		check(NumSpatialLayers == 1);
		check(NumTemporalLayers == 1);
		// In non-svc mode encoder doesn't provide reference list. Assume each frame
		// is reference and stored in buffer 0.
		RefBuf[0] = FrameBuf;
	}
}

template <typename TResource>
vpx_svc_ref_frame_config_t TVideoEncoderLibVpxVP9<TResource>::SetReferences(TArray<FScalableVideoController::FLayerFrameConfig>& InLayerFrames)
{
	vpx_svc_ref_frame_config_t RefConfig = {};
	for (const FScalableVideoController::FLayerFrameConfig& LayerFrame : InLayerFrames)
	{
		const auto& Buffers = LayerFrame.GetBuffers();
		check(Buffers.Num() <= 3);
		int Sid = LayerFrame.GetSpatialId();
		if (!Buffers.IsEmpty())
		{
			RefConfig.lst_fb_idx[Sid] = Buffers[0].Id;
			RefConfig.reference_last[Sid] = Buffers[0].bReferenced;
			if (Buffers[0].bUpdated)
			{
				RefConfig.update_buffer_slot[Sid] |= (1 << Buffers[0].Id);
			}
		}
		if (Buffers.Num() > 1)
		{
			RefConfig.gld_fb_idx[Sid] = Buffers[1].Id;
			RefConfig.reference_golden[Sid] = Buffers[1].bReferenced;
			if (Buffers[1].bUpdated)
			{
				RefConfig.update_buffer_slot[Sid] |= (1 << Buffers[1].Id);
			}
		}
		if (Buffers.Num() > 2)
		{
			RefConfig.alt_fb_idx[Sid] = Buffers[2].Id;
			RefConfig.reference_alt_ref[Sid] = Buffers[2].bReferenced;
			if (Buffers[2].bUpdated)
			{
				RefConfig.update_buffer_slot[Sid] |= (1 << Buffers[2].Id);
			}
		}
	}
	return RefConfig;
}

template <typename TResource>
vpx_svc_ref_frame_config_t TVideoEncoderLibVpxVP9<TResource>::SetReferences(bool bIsKeyPic, int FirstActiveSpatialLayerId)
{
	// kRefBufIdx, kUpdBufIdx need to be updated to support longer GOFs.
	check(Gof.NumFramesInGof <= Video::MaxTemporalStreams);

	vpx_svc_ref_frame_config_t RefConfig;
	memset(&RefConfig, 0, sizeof(vpx_svc_ref_frame_config_t));

	const size_t   NumTemporalRefs = FMath::Max(1, NumTemporalLayers - 1);
	const bool	   bIsInterLayerPredAllowed = InterLayerPrediction == EInterLayerPrediction::On || (InterLayerPrediction == EInterLayerPrediction::OnKeyPicture && bIsKeyPic);
	TOptional<int> LastUpdatedBufIdx;

	// Put temporal reference to LAST and spatial reference to GOLDEN. Update
	// frame buffer (i.e. store encoded frame) if current frame is a temporal
	// reference (i.e. it belongs to a low temporal layer) or it is a spatial
	// reference. In later case, always store spatial reference in the last
	// reference frame buffer.
	// For the case of 3 temporal and 3 spatial layers we need 6 frame buffers
	// for temporal references plus 1 buffer for spatial reference. 7 buffers
	// in total.

	for (int sl_idx = FirstActiveSpatialLayerId; sl_idx < NumActiveSpatialLayers; ++sl_idx)
	{
		const size_t CurrPicNum = bIsKeyPic ? 0 : PicsSinceKey + 1;
		const size_t GofIdx = CurrPicNum % Gof.NumFramesInGof;

		if (!bIsKeyPic)
		{
			// Set up temporal reference.
			uint8_t	  RefBufIdx[Video::MaxTemporalStreams] = { 0, 0, 0, 1 };
			const int BufIdx = sl_idx * NumTemporalRefs + RefBufIdx[GofIdx];

			// Last reference frame buffer is reserved for spatial reference. It is
			// not supposed to be used for temporal prediction.
			check(BufIdx < 7);

			const int PidDiff = CurrPicNum - RefBuf[BufIdx].PicNum;
			// Incorrect spatial layer may be in the buffer due to a key-frame.
			const bool bSameSpatialLayer = RefBuf[BufIdx].SpatialLayerId == sl_idx;
			bool	   bCorrectPid = false;
			if (bIsFlexibleMode)
			{
				bCorrectPid = PidDiff > 0 && PidDiff < 30;
			}
			else
			{
				// Below code assumes single temporal referecence.
				check(Gof.NumRefPics[GofIdx] == 1);
				bCorrectPid = PidDiff == Gof.PidDiff[GofIdx][0];
			}

			if (bSameSpatialLayer && bCorrectPid)
			{
				RefConfig.lst_fb_idx[sl_idx] = BufIdx;
				RefConfig.reference_last[sl_idx] = 1;
			}
			else
			{
				// This reference doesn't match with one specified by GOF. This can
				// only happen if spatial layer is enabled dynamically without key
				// frame. Spatial prediction is supposed to be enabled in this case.
				check(bIsInterLayerPredAllowed && sl_idx > FirstActiveSpatialLayerId);
			}
		}

		if (bIsInterLayerPredAllowed && sl_idx > FirstActiveSpatialLayerId)
		{
			// Set up spatial reference.
			check(LastUpdatedBufIdx.IsSet());
			RefConfig.gld_fb_idx[sl_idx] = *LastUpdatedBufIdx;
			RefConfig.reference_golden[sl_idx] = 1;
		}
		else
		{
			check(RefConfig.reference_last[sl_idx] != 0 || sl_idx == FirstActiveSpatialLayerId || InterLayerPrediction == EInterLayerPrediction::Off);
		}

		LastUpdatedBufIdx.Reset();

		if (Gof.TemporalIdx[GofIdx] < NumTemporalLayers - 1 || NumTemporalLayers == 1)
		{
			uint8_t UpdBufIdx[Video::MaxTemporalStreams] = { 0, 0, 1, 0 };
			LastUpdatedBufIdx = sl_idx * NumTemporalRefs + UpdBufIdx[GofIdx];

			// Ensure last frame buffer is not used for temporal prediction (it is
			// reserved for spatial reference).
			check(*LastUpdatedBufIdx < 7);
		}
		else if (bIsInterLayerPredAllowed)
		{
			LastUpdatedBufIdx = 7;
		}

		if (LastUpdatedBufIdx)
		{
			RefConfig.update_buffer_slot[sl_idx] = 1 << *LastUpdatedBufIdx;
		}
	}

	return RefConfig;
}

template <typename TResource>
bool TVideoEncoderLibVpxVP9<TResource>::SetSvcRates(FVideoEncoderConfigLibVpx const& Config, const FVideoBitrateAllocation& Allocation)
{
	TTuple<size_t, size_t> CurrentLayers = GetActiveLayers(Allocation);
	TTuple<size_t, size_t> NewLayers = GetActiveLayers(Allocation);

	const bool bLayerActivationRequiresKeyFrame = InterLayerPrediction == EInterLayerPrediction::Off || InterLayerPrediction == EInterLayerPrediction::OnKeyPicture;
	const bool bLowerLayersEnabled = NewLayers.Get<0>() < CurrentLayers.Get<0>();
	const bool bHigherLayersEnabled = NewLayers.Get<1>() > CurrentLayers.Get<1>();
	const bool bDisabledLayers = NewLayers.Get<0>() > CurrentLayers.Get<0>() || NewLayers.Get<1>() < CurrentLayers.Get<1>();

	if (bLowerLayersEnabled || (bHigherLayersEnabled && bLayerActivationRequiresKeyFrame))
	{
		bForceKeyFrame = true;
	}

	if (CurrentLayers != NewLayers)
	{
		bSsInfoNeeded = true;
	}

	VpxConfig->rc_target_bitrate = Allocation.GetSumBps() / 1000;

	if (ExplicitlyConfiguredSpatialLayers(Config))
	{
		for (size_t sl_idx = 0; sl_idx < NumSpatialLayers; ++sl_idx)
		{
			const bool bWasLayerActive = (VpxConfig->ss_target_bitrate[sl_idx] > 0);
			VpxConfig->ss_target_bitrate[sl_idx] = Allocation.GetSpatialLayerSumBitrate(sl_idx) / 1000;

			for (size_t tl_idx = 0; tl_idx < NumTemporalLayers; ++tl_idx)
			{
				VpxConfig->layer_target_bitrate[sl_idx * NumTemporalLayers + tl_idx] = Allocation.GetTemporalLayerSumBitrate(sl_idx, tl_idx) / 1000;
			}
		}
		FAVResult::Log(EAVResult::Success, FString::Printf(TEXT("VpxConfig->ss_target_bitrate = [ %d, %d, %d ]"), VpxConfig->ss_target_bitrate[0], VpxConfig->ss_target_bitrate[1], VpxConfig->ss_target_bitrate[2]), TEXT("LibVpx"));
	}
	else
	{
		float RateRatio[VPX_MAX_LAYERS] = { 0 };
		float Total = 0;
		for (int i = 0; i < NumSpatialLayers; ++i)
		{
			if (SvcParams->scaling_factor_num[i] <= 0 || SvcParams->scaling_factor_den[i] <= 0)
			{
				FAVResult::Log(EAVResult::Warning, TEXT("Scaling factors not specified!"), TEXT("LibVpx"));
				return false;
			}
			RateRatio[i] = static_cast<float>(SvcParams->scaling_factor_num[i]) / SvcParams->scaling_factor_den[i];
			Total += RateRatio[i];
		}

		for (int i = 0; i < NumSpatialLayers; ++i)
		{
			check(Total > 0);
			VpxConfig->ss_target_bitrate[i] = static_cast<unsigned int>(VpxConfig->rc_target_bitrate * RateRatio[i] / Total);
			if (NumTemporalLayers == 1)
			{
				VpxConfig->layer_target_bitrate[i] = VpxConfig->ss_target_bitrate[i];
			}
			else if (NumTemporalLayers == 2)
			{
				VpxConfig->layer_target_bitrate[i * NumTemporalLayers] = VpxConfig->ss_target_bitrate[i] * 2 / 3;
				VpxConfig->layer_target_bitrate[i * NumTemporalLayers + 1] = VpxConfig->ss_target_bitrate[i];
			}
			else if (NumTemporalLayers == 3)
			{
				VpxConfig->layer_target_bitrate[i * NumTemporalLayers] = VpxConfig->ss_target_bitrate[i] / 2;
				VpxConfig->layer_target_bitrate[i * NumTemporalLayers + 1] = VpxConfig->layer_target_bitrate[i * NumTemporalLayers] + (VpxConfig->ss_target_bitrate[i] / Video::MaxTemporalStreams);
				VpxConfig->layer_target_bitrate[i * NumTemporalLayers + 2] = VpxConfig->ss_target_bitrate[i];
			}
			else
			{
				FAVResult::Log(EAVResult::Warning, FString::Printf(TEXT("Unsupported number of temporal layers: %d"), NumTemporalLayers), TEXT("LibVpx"));
				return false;
			}
		}
	}

	NumActiveSpatialLayers = 0;
	FirstActiveLayer = 0;
	bool bSeenActiveLayer = false;
	bool bExpectNoMoreActiveLayers = false;
	for (int i = 0; i < NumSpatialLayers; ++i)
	{
		if (VpxConfig->ss_target_bitrate[i] > 0)
		{
			if (bExpectNoMoreActiveLayers)
			{
				// UE_LOGFMT(LogPixelStreamingEpicRtc, Warning, "Only middle layer is deactivated");
				FAVResult::Log(EAVResult::Warning, TEXT("Only middle layer is deactivated"), TEXT("LibVpx"));
			}
			if (!bSeenActiveLayer)
			{
				FirstActiveLayer = i;
			}
			NumActiveSpatialLayers = i + 1;
			bSeenActiveLayer = true;
		}
		else
		{
			bExpectNoMoreActiveLayers = bSeenActiveLayer;
		}
	}

	if (bSeenActiveLayer && PerformanceFlags.bUsePerLayerSpeed)
	{
		bool bDenoiserOn = this->AppliedConfig.bDenoisingOn && PerformanceFlagsBySpatialIndex[NumActiveSpatialLayers - 1].bAllowDenoising;
		::vpx_codec_control(Encoder.Get(), VP9E_SET_NOISE_SENSITIVITY, bDenoiserOn ? 1 : 0);
	}

	if (bHigherLayersEnabled && !bForceKeyFrame)
	{
		// Prohibit drop of all layers for the next frame, so newly enabled
		// layer would have a valid spatial reference.
		for (size_t i = 0; i < NumSpatialLayers; ++i)
		{
			SvcDropFrame->framedrop_thresh[i] = 0;
		}
		bForceAllActiveLayers = true;
	}

	if (SvcController)
	{
		for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
		{
			// Bitrates in `layer_target_bitrate` are accumulated for each temporal
			// layer but in `VideoBitrateAllocation` they should be separated.
			int PreviousBitrateKbps = 0;
			for (int Tid = 0; Tid < NumTemporalLayers; ++Tid)
			{
				int AccumulatedBitrateKbps = VpxConfig->layer_target_bitrate[Sid * NumTemporalLayers + Tid];
				int SingleLayerBitrateKbps = AccumulatedBitrateKbps - PreviousBitrateKbps;
				check(SingleLayerBitrateKbps >= 0);
				CurrentBitrateAllocation.SetBitrate(Sid, Tid, SingleLayerBitrateKbps * 1000);
				PreviousBitrateKbps = AccumulatedBitrateKbps;
			}
		}
		SvcController->OnRatesUpdated(CurrentBitrateAllocation);
	}
	else
	{
		CurrentBitrateAllocation = Allocation;
	}
	bVpxConfigChanged = true;

	return true;
}

template <typename TResource>
bool TVideoEncoderLibVpxVP9<TResource>::ExplicitlyConfiguredSpatialLayers(FVideoEncoderConfigLibVpx const& Config) const
{
	// We check target_bitrate_bps of the 0th Layer to see if the spatial layers
	// (i.e. bitrates) were explicitly configured.
	return Config.SpatialLayers[0].TargetBitrate > 0;
}

template <typename TResource>
void TVideoEncoderLibVpxVP9<TResource>::UpdatePerformanceFlags(FVideoEncoderConfigLibVpx const& Config)
{
	TSortedMap<int, FParameterSet> ParamsByResolution;
	ParamsByResolution = PerformanceFlags.SettingsByResolution;

	const auto FindSpeed = [&ParamsByResolution](int MinPixelCount) {
		check(!(ParamsByResolution.Num() == 0));

		TArray<int> ResolutionArray;
		ParamsByResolution.GenerateKeyArray(ResolutionArray);
		size_t Index = Algo::UpperBound(ResolutionArray, MinPixelCount);
		if (Index > 0)
		{
			--Index;
		}

		return ParamsByResolution[ResolutionArray[Index]];
		};
	PerformanceFlagsBySpatialIndex.Empty();

	if (bIsSvc)
	{
		for (int si = 0; si < NumSpatialLayers; ++si)
		{
			PerformanceFlagsBySpatialIndex.Add(FindSpeed(Config.SpatialLayers[si].Width * Config.SpatialLayers[si].Height));
		}
	}
	else
	{
		PerformanceFlagsBySpatialIndex.Add(FindSpeed(Config.Width * Config.Height));
	}
}

template <typename TResource>
void TVideoEncoderLibVpxVP9<TResource>::MaybeRewrapRawWithFormat(const vpx_img_fmt Format)
{
	if (!RawImage.IsValid())
	{
		RawImage = TUniquePtr<vpx_image_t, LibVpxUtil::FImageDeleter>(::vpx_img_wrap(nullptr, Format, this->AppliedConfig.Width, this->AppliedConfig.Height, 1, nullptr), LibVpxUtil::FImageDeleter());
	}
	else if (RawImage->fmt != Format)
	{
		// UE_LOGFMT(LogPixelStreamingEpicRtc, Log, "Switching VP9 encoder pixel format to {0}", Format == VPX_IMG_FMT_NV12 ? "NV12" : "I420");
		::vpx_img_free(RawImage.Get());
		RawImage = TUniquePtr<vpx_image_t, LibVpxUtil::FImageDeleter>(::vpx_img_wrap(nullptr, Format, this->AppliedConfig.Width, this->AppliedConfig.Height, 1, nullptr), LibVpxUtil::FImageDeleter());
	}
	// else no-op since the image is already in the right format.
}

template <typename TResource>
TTuple<size_t, size_t> TVideoEncoderLibVpxVP9<TResource>::GetActiveLayers(const FVideoBitrateAllocation& Allocation)
{
	for (size_t sl_idx = 0; sl_idx < Video::MaxSpatialLayers; ++sl_idx)
	{
		if (Allocation.GetSpatialLayerSumBitrate(sl_idx) > 0)
		{
			size_t LastLayer = sl_idx + 1;
			while (LastLayer < Video::MaxSpatialLayers && Allocation.GetSpatialLayerSumBitrate(LastLayer) > 0)
			{
				++LastLayer;
			}
			return MakeTuple(sl_idx, LastLayer);
		}
	}
	return { 0, 0 };
}
