// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LibVpx.h"

class LibVpxUtil
{
public:
	struct FImageDeleter
	{
		void operator()(vpx_image_t* Image) const
		{
			::vpx_img_free(Image);
		}
	};

	struct FCodecContextDeleter
	{
		void operator()(vpx_codec_ctx_t* Context) const
		{
			::vpx_codec_destroy(Context);
		}
	};

	static void CopyI420(const uint8_t* RESTRICT SrcY,
		int										 SrcStrideY,
		const uint8_t* RESTRICT					 SrcU,
		int										 SrcStrideU,
		const uint8_t* RESTRICT					 SrcV,
		int										 SrcStrideV,
		uint8_t* RESTRICT						 DstY,
		int										 DstStrideY,
		uint8_t* RESTRICT						 DstU,
		int										 DstStrideU,
		uint8_t* RESTRICT						 DstV,
		int										 DstStrideV,
		int										 Width,
		int										 Height);

	static void CopyPlane(const uint8_t* RESTRICT SrcY,
		int										  SrcStrideY,
		uint8_t* RESTRICT						  DstY,
		int										  DstStrideY,
		int										  Width,
		int										  Height);
};